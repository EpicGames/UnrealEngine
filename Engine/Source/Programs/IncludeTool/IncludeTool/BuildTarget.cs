// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Type of binary file
	/// </summary>
	enum BuildBinaryType
	{
		Executable,
		StaticLibrary,
		DynamicLinkLibrary,
	}

	/// <summary>
	/// Wrapper class for an target binary exported by UBT
	/// </summary>
	class BuildBinary
	{
		public FileReference File;
		public BuildBinaryType Type;
		public HashSet<BuildModule> Modules;

		public override string ToString()
		{
			return File.ToString();
		}
	}

	/// <summary>
	/// Type of PCH
	/// </summary>
	enum BuildModulePCHUsage
	{
		Default,
		NoPCHs,
		NoSharedPCHs,
		UseSharedPCHs,
		UseExplicitOrSharedPCHs
	}

	/// <summary>
	/// Wrapper class for a target module exported by UBT
	/// </summary>
	class BuildModule
	{
		public string Name;
		public BuildModulePCHUsage PCHUsage;
		public FileReference PrivatePCH;
		public FileReference SharedPCH;
		public DirectoryReference Directory;
		public HashSet<BuildModule> PublicDependencyModules;
		public HashSet<BuildModule> PublicIncludePathModules;
		public HashSet<BuildModule> PrivateDependencyModules;
		public HashSet<BuildModule> PrivateIncludePathModules;
		public HashSet<BuildModule> CircularlyReferencedModules;

		public IEnumerable<BuildModule> Dependencies
		{
			get { return PublicDependencyModules.Union(PrivateDependencyModules); }
		}

		public IEnumerable<BuildModule> NonCircularDependencies
		{
			get { return Dependencies.Except(CircularlyReferencedModules); }
		}

		public override string ToString()
		{
			return Name;
		}
	}

	/// <summary>
	/// Wrapper class for an build target exported by UBT
	/// </summary>
	class BuildTarget
	{
		public string Name;
		public string Configuration;
		public string Platform;
		public FileReference ProjectFile;
		public List<BuildBinary> Binaries = new List<BuildBinary>();
		public List<BuildModule> Modules = new List<BuildModule>();
		public Dictionary<string, BuildModule> NameToModule = new Dictionary<string, BuildModule>();

		public static BuildTarget Read(FileReference File)
		{
			JsonObject Object = JsonObject.Read(File);

			BuildTarget Target = new BuildTarget();
			Target.Name = Object.GetStringField("Name");
			Target.Configuration = Object.GetStringField("Configuration");
			Target.Platform = Object.GetStringField("Platform");

			string ProjectFile;
			if (Object.TryGetStringField("ProjectFile", out ProjectFile))
			{
				Target.ProjectFile = new FileReference(ProjectFile);
			}

			JsonObject Modules = Object.GetObjectField("Modules");
			Target.NameToModule = Modules.KeyNames.ToDictionary(x => x, x => new BuildModule { Name = x }, StringComparer.OrdinalIgnoreCase);
			Target.Modules.AddRange(Target.NameToModule.Values);

			foreach (BuildModule Module in Target.Modules)
			{
				JsonObject ModuleObject = Modules.GetObjectField(Module.Name);

				string PrivatePCH;
				if (ModuleObject.TryGetStringField("PrivatePCH", out PrivatePCH))
				{
					Module.PrivatePCH = new FileReference(PrivatePCH);
				}

				string SharedPCH;
				if (ModuleObject.TryGetStringField("SharedPCH", out SharedPCH))
				{
					Module.SharedPCH = new FileReference(SharedPCH);
				}

				Module.PCHUsage = ModuleObject.GetEnumField<BuildModulePCHUsage>("PCHUsage");
				Module.Directory = new DirectoryReference(ModuleObject.GetStringField("Directory"));

				Module.PublicDependencyModules = new HashSet<BuildModule>(ModuleObject.GetStringArrayField("PublicDependencyModules").Select(x => Target.NameToModule[x]));
				Module.PublicIncludePathModules = new HashSet<BuildModule>(ModuleObject.GetStringArrayField("PublicIncludePathModules").Select(x => Target.NameToModule[x]));
				Module.PrivateDependencyModules = new HashSet<BuildModule>(ModuleObject.GetStringArrayField("PrivateDependencyModules").Select(x => Target.NameToModule[x]));
				Module.PrivateIncludePathModules = new HashSet<BuildModule>(ModuleObject.GetStringArrayField("PrivateIncludePathModules").Select(x => Target.NameToModule[x]));
				Module.CircularlyReferencedModules = new HashSet<BuildModule>(ModuleObject.GetStringArrayField("CircularlyReferencedModules").Select(x => Target.NameToModule[x]));
			}

			foreach (JsonObject BinaryObject in Object.GetObjectArrayField("Binaries"))
			{
				BuildBinary Binary = new BuildBinary();
				Binary.File = new FileReference(BinaryObject.GetStringField("File"));
				Binary.Type = BinaryObject.GetEnumField<BuildBinaryType>("Type");
				Binary.Modules = new HashSet<BuildModule>(BinaryObject.GetStringArrayField("Modules").Select(x => Target.NameToModule[x]));
				Target.Binaries.Add(Binary);
			}

			return Target;
		}

		public override string ToString()
		{
			return Name;
		}
	}
}
