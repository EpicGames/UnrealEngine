// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Xml;

namespace IncludeTool
{
	/// <summary>
	/// The type of the compiler
	/// </summary>
	public enum CompilerType
	{
		Unknown,
		Clang,
		VisualC,
	}

	/// <summary>
	/// Encapsulates a compiler option
	/// </summary>
	[Serializable]
	public class CompileOption
	{
		/// <summary>
		/// The option name
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// The option value. May be null.
		/// </summary>
		public readonly string Value;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InName">The option name</param>
		/// <param name="InValue">The option value. May be null.</param>
		public CompileOption(string InName, string InValue)
		{
			Name = InName;
			Value = InValue;
		}

		/// <summary>
		/// Formats the option as it would appear on the command line.
		/// </summary>
		/// <returns>String containing the option</returns>
		public override string ToString()
		{
			if(Value == null)
			{
				return Name;
			}
			else if(Name == "/Fo" || Name == "/Yc")
			{
				return Name + Utility.QuoteArgument(Value);
			}
			else
			{
				return Name + " " + Utility.QuoteArgument(Value);
			}
		}
	}

	/// <summary>
	/// Environment used to compile a file
	/// </summary>
	[Serializable]
	class CompileEnvironment
	{
		/// <summary>
		/// Path to the compiler executable
		/// </summary>
		public FileReference Compiler;

		/// <summary>
		/// The type of compiler
		/// </summary>
		public CompilerType CompilerType;

		/// <summary>
		/// All the options, with the exception of definitions and include paths
		/// </summary>
		public List<CompileOption> Options = new List<CompileOption>();

		/// <summary>
		/// List of macro definitions
		/// </summary>
		public List<string> Definitions = new List<string>();

		/// <summary>
		/// List of include paths
		/// </summary>
		public List<DirectoryReference> IncludePaths = new List<DirectoryReference>();

		/// <summary>
		/// List of force-included headers
		/// </summary>
		public List<FileReference> ForceIncludeFiles = new List<FileReference>();

		/// <summary>
		/// Default constructor
		/// </summary>
		public CompileEnvironment(FileReference InCompiler, CompilerType InCompilerType)
		{
			Compiler = InCompiler;
			CompilerType = InCompilerType;
		}

		/// <summary>
		/// Clone another compile environment
		/// </summary>
		/// <param name="Other">The compile environment to copy</param>
		public CompileEnvironment(CompileEnvironment Other)
		{
			Compiler = Other.Compiler;
			CompilerType = Other.CompilerType;
			Options = new List<CompileOption>(Other.Options);
			Definitions = new List<string>(Other.Definitions);
			IncludePaths = new List<DirectoryReference>(Other.IncludePaths);
			ForceIncludeFiles = new List<FileReference>(Other.ForceIncludeFiles);
		}

		/// <summary>
		/// Gets the full command line to compile with, with the exception of any source files
		/// </summary>
		/// <returns>The complete command line</returns>
		public string GetCommandLine()
		{
			List<string> Arguments = new List<string>();
			Arguments.AddRange(Options.Select(x => Utility.QuoteArgument(x.ToString())));
			if(CompilerType == CompilerType.Clang)
			{
				Arguments.AddRange(Definitions.Select(x => "-D " + x));
				Arguments.AddRange(IncludePaths.Select(x => "-I \"" + x + "\""));
				Arguments.AddRange(ForceIncludeFiles.Select(x => "-include \"" + x + "\""));
			}
			else
			{
				Arguments.AddRange(Definitions.Select(x => "/D " + x));
				Arguments.AddRange(IncludePaths.Select(x => "/I \"" + x + "\""));
				Arguments.AddRange(ForceIncludeFiles.Select(x => "/FI\"" + x + "\""));
			}
			return String.Join(" ", Arguments);
		}

		/// <summary>
		/// Enumerates all the options with the given name
		/// </summary>
		/// <param name="InName">Option to look for</param>
		/// <returns>Sequence of the matching values for the given name</returns>
		public IEnumerable<string> GetOptionValues(string InName)
		{
			foreach (CompileOption Option in Options)
			{
				if (Option.Name == InName)
				{
					yield return Option.Value;
				}
			}
		}

		/// <summary>
		/// Writes a response file which will compile the given source file
		/// </summary>
		/// <param name="ResponseFile">The response file to write to</param>
		/// <param name="SourceFile">The source file to compile</param>
		public void WriteResponseFile(FileReference ResponseFile, FileReference SourceFile)
		{
			using (StreamWriter Writer = new StreamWriter(ResponseFile.FullName))
			{
				foreach (CompileOption Option in Options)
				{
					if(CompilerType == CompilerType.Clang)
					{
						Writer.WriteLine(Option.ToString().Replace('\\', '/'));
					}
					else
					{
						Writer.WriteLine(Option.ToString());
					}
				}

				foreach (string Definition in Definitions)
				{
					if(CompilerType == CompilerType.Clang)
					{
						Writer.WriteLine("-D {0}", Utility.QuoteArgument(Definition));
					}
					else
					{
						Writer.WriteLine("/D {0}", Utility.QuoteArgument(Definition));
					}
				}

				foreach (DirectoryReference IncludePath in IncludePaths)
				{
					if(CompilerType == CompilerType.Clang)
					{
						Writer.WriteLine("-I {0}", Utility.QuoteArgument(IncludePath.FullName.Replace('\\', '/')));
					}
					else
					{
						Writer.WriteLine("/I {0}", Utility.QuoteArgument(IncludePath.FullName));
					}
				}

				foreach (FileReference ForceIncludeFile in ForceIncludeFiles)
				{
					if(CompilerType == CompilerType.Clang)
					{
						Writer.WriteLine("-include {0}", Utility.QuoteArgument(ForceIncludeFile.FullName.Replace('\\', '/')));
					}
					else
					{
						Writer.WriteLine("/FI{0}", Utility.QuoteArgument(ForceIncludeFile.FullName));
					}
				}

				if(CompilerType == CompilerType.Clang)
				{
					Writer.WriteLine(Utility.QuoteArgument(SourceFile.FullName.Replace('\\', '/')));
				}
				else
				{
					Writer.WriteLine("/Tp\"{0}\"", SourceFile.FullName);
				}
			}
		}

		/// <summary>
		/// Reads an exported XGE task list
		/// </summary>
		/// <param name="TaskListFile">File to read from</param>
		/// <param name="FileToEnvironment">Mapping from source file to compile environment</param>
		public static void ReadTaskList(FileReference TaskListFile, DirectoryReference BaseDir, out Dictionary<FileReference, CompileEnvironment> FileToEnvironment)
		{
			XmlDocument Document = new XmlDocument();
			Document.Load(TaskListFile.FullName);

			// Read the standard include paths from the INCLUDE environment variable in the script
			List<DirectoryReference> StandardIncludePaths = new List<DirectoryReference>();
			foreach (XmlNode Node in Document.SelectNodes("BuildSet/Environments/Environment/Variables/Variable"))
			{
				XmlAttribute NameAttr = Node.Attributes["Name"];
				if(NameAttr != null && String.Compare(NameAttr.InnerText, "INCLUDE") == 0)
				{
					foreach(string IncludePath in Node.Attributes["Value"].InnerText.Split(';'))
					{
						StandardIncludePaths.Add(new DirectoryReference(IncludePath));
					}
				}
			}

			// Read all the individual compiles
			Dictionary<FileReference, CompileEnvironment> NewFileToEnvironment = new Dictionary<FileReference, CompileEnvironment>();
			foreach (XmlNode Node in Document.SelectNodes("BuildSet/Environments/Environment/Tools/Tool"))
			{
				XmlAttribute ToolPathAttr = Node.Attributes["Path"];
				if (ToolPathAttr != null)
				{
					// Get the full path to the tool
					FileReference ToolLocation = new FileReference(ToolPathAttr.InnerText);

					// Get the compiler type
					CompilerType CompilerType;
					if(GetCompilerType(ToolLocation, out CompilerType))
					{
						string Name = Node.Attributes["Name"].InnerText;
						string Params = Node.Attributes["Params"].InnerText;

						// Construct the compile environment
						CompileEnvironment Environment = new CompileEnvironment(ToolLocation, CompilerType);

						// Parse a list of tokens
						List<string> Tokens = new List<string>();
						ParseArgumentTokens(CompilerType, Params, Tokens, BaseDir);

						// Parse it into a list of options, removing any that don't apply
						List<FileReference> SourceFiles = new List<FileReference>();
						List<FileReference> OutputFiles = new List<FileReference>();
						for (int Idx = 0; Idx < Tokens.Count; Idx++)
						{
							if(Tokens[Idx] == "/Fo" || Tokens[Idx] == "/Fp" || Tokens[Idx] == "-o")
							{
								string OutputPath = Tokens[++Idx];
								if (!Path.IsPathRooted(OutputPath))
								{
									OutputPath = Path.Combine(BaseDir.FullName, OutputPath);
								}
								OutputFiles.Add(new FileReference(OutputPath));
							}
							else if(Tokens[Idx].StartsWith("/Fo") || Tokens[Idx].StartsWith("/Fp"))
							{
								string OutputPath = Tokens[Idx].Substring(3);
								if (!Path.IsPathRooted(OutputPath))
								{
									OutputPath = Path.Combine(BaseDir.FullName, OutputPath);
								}
								OutputFiles.Add(new FileReference(OutputPath));
							}
							else if (Tokens[Idx] == "/D" || Tokens[Idx] == "-D")
							{
								Environment.Definitions.Add(Tokens[++Idx]);
							}
							else if(Tokens[Idx].StartsWith("/D"))
							{
								Environment.Definitions.Add(Tokens[Idx].Substring(2));
							}
							else if (Tokens[Idx] == "/I" || Tokens[Idx] == "/external:I" || Tokens[Idx] == "-I" || Tokens[Idx] == "/imsvc" || Tokens[Idx] == "-isystem")
							{
								string IncludePath = Tokens[++Idx].Replace("//", "/");
								if (!Path.IsPathRooted(IncludePath))
								{
									IncludePath = Path.Combine(BaseDir.FullName, IncludePath);
								}
								Environment.IncludePaths.Add(new DirectoryReference(IncludePath.ToLowerInvariant()));
							}
							else if (Tokens[Idx].StartsWith("-I"))
							{
								string IncludePath = Tokens[Idx].Substring(2).Replace("//", "/");
								if (!Path.IsPathRooted(IncludePath))
								{
									IncludePath = Path.Combine(BaseDir.FullName, IncludePath);
								}
								Environment.IncludePaths.Add(new DirectoryReference(IncludePath.ToLowerInvariant()));
							}
							else if (Tokens[Idx].StartsWith("-isystem"))
							{
								string IncludePath = Tokens[Idx].Substring(8).Replace("//", "/");
								if (!Path.IsPathRooted(IncludePath))
								{
									IncludePath = Path.Combine(BaseDir.FullName, IncludePath);
								}
								Environment.IncludePaths.Add(new DirectoryReference(IncludePath.ToLowerInvariant()));
							}
							else if(Tokens[Idx].StartsWith("/FI"))
							{
								string ForceIncludePath = Tokens[Idx].Substring(3);
								if (!Path.IsPathRooted(ForceIncludePath))
								{
									ForceIncludePath = Path.Combine(BaseDir.FullName, ForceIncludePath);
								}
								Environment.ForceIncludeFiles.Add(new FileReference(ForceIncludePath));
							}
							else if(Tokens[Idx] == "-include")
							{
								string ForceIncludePath = Tokens[++Idx];
								if (!Path.IsPathRooted(ForceIncludePath))
								{
									ForceIncludePath = Path.Combine(BaseDir.FullName, ForceIncludePath);
								}
								Environment.ForceIncludeFiles.Add(new FileReference(ForceIncludePath));
							}
							else if (Tokens[Idx] == "-Xclang" || Tokens[Idx] == "-target" || Tokens[Idx] == "--target" || Tokens[Idx] == "-x" || Tokens[Idx] == "-o")
							{
								Environment.Options.Add(new CompileOption(Tokens[Idx], Tokens[++Idx]));
							}
							else if (Tokens[Idx].StartsWith("/") || Tokens[Idx].StartsWith("-"))
							{
								Environment.Options.Add(new CompileOption(Tokens[Idx], null));
							}
							else
							{
								SourceFiles.Add(FileReference.Combine(BaseDir, Tokens[Idx]));
							}
						}

						// Make sure we're not compiling a precompiled header
						if(!OutputFiles.Any(x => x.HasExtension(".gch")) && !Environment.Options.Any(x => x.Name.StartsWith("/Yc")))
						{
							// Add all the standard include paths
							Environment.IncludePaths.AddRange(StandardIncludePaths);

							// Add to the output map if there are any source files. Use the extension to distinguish between a source file and linker invocation on Clang.
							foreach (FileReference SourceFile in SourceFiles)
							{
								if(!SourceFile.HasExtension(".a"))
								{
									if(NewFileToEnvironment.ContainsKey(SourceFile))
									{
										Console.WriteLine("Source file {0} is compiled with multiple environments", SourceFile);
									}
									else
									{
										NewFileToEnvironment.Add(SourceFile, Environment);
									}
								}
							}
						}
					}
				}
			}
			FileToEnvironment = NewFileToEnvironment;
		}

		/// <summary>
		/// Determine the compiler type
		/// </summary>
		/// <param name="Compiler">Path to the compiler</param>
		/// <param name="CompilerType">Type of the compiler</param>
		/// <returns>True if the path is a valid known compiler, and CompilerType is set</returns>
		static bool GetCompilerType(FileReference Compiler, out CompilerType CompilerType)
		{
			string FileName = Compiler.GetFileName().ToLowerInvariant();
			if(FileName == "cl.exe")
			{
				CompilerType = CompilerType.VisualC;
				return true;
			}
			else if(FileName == "clang++.exe")
			{
				CompilerType = CompilerType.Clang;
				return true;
			}
			else
			{
				CompilerType = CompilerType.Unknown;
				return false;
			}
		}

		/// <summary>
		/// Parse a parameter list into a series of tokens, reading response files as appropriate
		/// </summary>
		/// <param name="Text">The line of text to parse</param>
		/// <param name="Tokens">List to be filled with the parsed tokens</param>
		/// <param name="BaseDir">Base directory to root relative pathed response files from</param>
		static void ParseArgumentTokens(CompilerType CompilerType, string Text, List<string> Tokens, DirectoryReference BaseDir)
		{
			for (int Idx = 0; Idx < Text.Length; Idx++)
			{
				if (!Char.IsWhiteSpace(Text[Idx]))
				{
					// Read until the end of the token
					StringBuilder Token = new StringBuilder();
					while (Idx < Text.Length && !Char.IsWhiteSpace(Text[Idx]))
					{
						if (Text[Idx] == '"')
						{
							Idx++;
							while (Text[Idx] != '"')
							{
								if(Idx + 1 < Text.Length && Text[Idx] == '\\' && Text[Idx + 1] == '\"' && CompilerType == CompilerType.Clang)
								{
									Idx++;
								}
								Token.Append(Text[Idx++]);
							}
							Idx++;
						}
						else
						{
							Token.Append(Text[Idx++]);
						}
					}

					// Add the token to the list. If it's a response file, recursively parse the tokens out of that.
					if(Token[0] == '@')
					{
						string ResponsePath = Token.ToString().Substring(1);
						if (!Path.IsPathRooted(ResponsePath))
						{
							ResponsePath = Path.Combine(BaseDir.FullName, ResponsePath);
						}
						ParseArgumentTokens(CompilerType, File.ReadAllText(ResponsePath), Tokens, BaseDir);
					}
					else
					{
						Tokens.Add(Token.ToString());
					}
				}
			}
		}

		/// <summary>
		/// Returns the full command line for this compile environment
		/// </summary>
		/// <returns>String containing the full command line</returns>
		public override string ToString()
		{
			return GetCommandLine();
		}
	}
}
