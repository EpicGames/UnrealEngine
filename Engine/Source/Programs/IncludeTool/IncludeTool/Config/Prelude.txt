; This file is used to generate Prelude.h, which sets up the initial environment for the preprocessor.

; Standard macros
__STDC__
__STDC_VERSION__
__STDC_HOSTED__
__cplusplus
__TIMESTAMP__

; Microsoft defines, taken from https://msdn.microsoft.com/en-us/library/b0084kay.aspx
_ATL_VER
__AVX__
__AVX2__
_CHAR_UNSIGNED
__CLR_VER
__cplusplus_cli
__cplusplus_winrt
_CPPRTTI
_CPPUNWIND
_DEBUG
_DLL
_INTEGRAL_MAX_BITS
_M_AMD64
_M_ARM
_M_CEE
_M_CEE_PURE
_M_CEE_SAFE
_M_IX86
_M_ARM_FP
_M_IX86_FP
_M_X64
_MANAGED
_MFC_VER
_MSC_BUILD
_MSC_EXTENSIONS
_MSC_FULL_VER
_MSC_VER
__MSVC_RUNTIME_CHECKS
_MT
_NATIVE_WCHAR_T_DEFINED
_OPENMP
_VC_NODEFAULTLIB
_WCHAR_T_DEFINED
_WIN32
_WIN64

; Microsoft defines, undocumented but referenced by headers
_M_FP_FAST

; GCC defines, taken from https://gcc.gnu.org/onlinedocs/cpp/Common-Predefined-Macros.html#Common-Predefined-Macros
__OBJC__
__ASSEMBLER__
__GFORTRAN__
__GNUC__
__GNUC_MINOR__
__GNUC_PATCHLEVEL__
__GNUG__
__STRICT_ANSI__
__BASE_FILE__
__INCLUDE_LEVEL__
__ELF__
__VERSION__
__OPTIMIZE__
__OPTIMIZE_SIZE__
__NO_INLINE__
__GNUC_GNU_INLINE__
__GNUC_STDC_INLINE__
__CHAR_UNSIGNED__
__WCHAR_UNSIGNED__
__REGISTER_PREFIX__
__USER_LABEL_PREFIX__
__SIZE_TYPE__
__PTRDIFF_TYPE__
__WCHAR_TYPE__
__WINT_TYPE__
__INTMAX_TYPE__
__UINTMAX_TYPE__
__SIG_ATOMIC_TYPE__
__INT8_TYPE__
__INT16_TYPE__
__INT32_TYPE__
__INT64_TYPE__
__UINT8_TYPE__
__UINT16_TYPE__
__UINT32_TYPE__
__UINT64_TYPE__
__INT_LEAST8_TYPE__
__INT_LEAST16_TYPE__
__INT_LEAST32_TYPE__
__INT_LEAST64_TYPE__
__UINT_LEAST8_TYPE__
__UINT_LEAST16_TYPE__
__UINT_LEAST32_TYPE__
__UINT_LEAST64_TYPE__
__INT_FAST8_TYPE__
__INT_FAST16_TYPE__
__INT_FAST32_TYPE__
__INT_FAST64_TYPE__
__UINT_FAST8_TYPE__
__UINT_FAST16_TYPE__
__UINT_FAST32_TYPE__
__UINT_FAST64_TYPE__
__INTPTR_TYPE__
__UINTPTR_TYPE__
__CHAR_BIT__
__SCHAR_MAX__
__WCHAR_MAX__
__SHRT_MAX__
__INT_MAX__
__LONG_MAX__
__LONG_LONG_MAX__
__WINT_MAX__
__SIZE_MAX__
__PTRDIFF_MAX__
__INTMAX_MAX__
__UINTMAX_MAX__
__SIG_ATOMIC_MAX__
__INT8_MAX__
__INT16_MAX__
__INT32_MAX__
__INT64_MAX__
__UINT8_MAX__
__UINT16_MAX__
__UINT32_MAX__
__UINT64_MAX__
__INT_LEAST8_MAX__
__INT_LEAST16_MAX__
__INT_LEAST32_MAX__
__INT_LEAST64_MAX__
__UINT_LEAST8_MAX__
__UINT_LEAST16_MAX__
__UINT_LEAST32_MAX__
__UINT_LEAST64_MAX__
__INT_FAST8_MAX__
__INT_FAST16_MAX__
__INT_FAST32_MAX__
__INT_FAST64_MAX__
__UINT_FAST8_MAX__
__UINT_FAST16_MAX__
__UINT_FAST32_MAX__
__UINT_FAST64_MAX__
__INTPTR_MAX__
__UINTPTR_MAX__
__WCHAR_MIN__
__WINT_MIN__
__SIG_ATOMIC_MIN__
__INT8_C
__INT16_C
__INT32_C
__INT64_C
__UINT8_C
__UINT16_C
__UINT32_C
__UINT64_C
__INTMAX_C
__UINTMAX_C
__SIZEOF_INT__
__SIZEOF_LONG__
__SIZEOF_LONG_LONG__
__SIZEOF_SHORT__
__SIZEOF_POINTER__
__SIZEOF_FLOAT__
__SIZEOF_DOUBLE__
__SIZEOF_LONG_DOUBLE__
__SIZEOF_SIZE_T__
__SIZEOF_WCHAR_T__
__SIZEOF_WINT_T__
__SIZEOF_PTRDIFF_T__
__BYTE_ORDER__
__ORDER_LITTLE_ENDIAN__
__ORDER_BIG_ENDIAN__
__ORDER_PDP_ENDIAN__
__FLOAT_WORD_ORDER__
__DEPRECATED
__EXCEPTIONS
__GXX_RTTI
__USING_SJLJ_EXCEPTIONS__
__GXX_EXPERIMENTAL_CXX0X__
__GXX_WEAK__
__NEXT_RUNTIME__
__LP64__
_LP64
__SSP__
__SSP_ALL__
__SSP_STRONG__
__SSP_EXPLICIT__
__SANITIZE_ADDRESS__
__SANITIZE_THREAD__
__TIMESTAMP__
__GCC_HAVE_SYNC_COMPARE_AND_SWAP_1
__GCC_HAVE_SYNC_COMPARE_AND_SWAP_2
__GCC_HAVE_SYNC_COMPARE_AND_SWAP_4
__GCC_HAVE_SYNC_COMPARE_AND_SWAP_8
__GCC_HAVE_SYNC_COMPARE_AND_SWAP_16
__GCC_HAVE_DWARF2_CFI_ASM
__FP_FAST_FMA
__FP_FAST_FMAF
__FP_FAST_FMAL
__GCC_IEC_559
__GCC_IEC_559_COMPLEX
__NO_MATH_ERRNO__
