// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Stores information about a circular include path. All headers processed by the tool should form a direct hierarchy, whereby one
	/// header always includes another. If two headers include each other, we can't reason about what the IWYU derivation for each should 
	/// be in case they're each included by different files.
	/// </summary>
	class IncludeCycle
	{
		/// <summary>
		/// Array of files in this cycle
		/// </summary>
		public SourceFile[] Files;

		/// <summary>
		/// List of observed starting points for this cycle, along with a counter for its number of appearances.
		/// </summary>
		public List<KeyValuePair<SourceFile, int>> StartingPoints;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="InFiles">List of files in this cycle</param>
		public IncludeCycle(IEnumerable<SourceFile> InFiles)
		{
			Files = InFiles.ToArray();
			StartingPoints = new List<KeyValuePair<SourceFile, int>>();
			StartingPoints.Add(new KeyValuePair<SourceFile, int>(Files[0], 1));
		}

		/// <summary>
		/// Adds a new starting point for this cycle. If the starting point already exists, increases the counter for number of times it's been seen.
		/// </summary>
		/// <param name="File">The first file in the cycle.</param>
		public void AddStartingPoint(SourceFile File)
		{
			// Increment the counter for this starting point
			for(int Idx = 0; ; Idx++)
			{
				if(Idx == StartingPoints.Count)
				{
					StartingPoints.Add(new KeyValuePair<SourceFile,int>(File, 1));
					break;
				}
				else if(StartingPoints[Idx].Key == File)
				{
					StartingPoints[Idx] = new KeyValuePair<SourceFile,int>(StartingPoints[Idx].Key, StartingPoints[Idx].Value + 1);
					break;
				}
			}

			// Find the new largest starting point
			int LargestIdx = 0;
			for(int Idx = 1; Idx < StartingPoints.Count; Idx++)
			{
				if(StartingPoints[Idx].Value > StartingPoints[LargestIdx].Value)
				{
					LargestIdx = Idx;
				}
			}

			// If it's not the same as the current one, reorder the sequence
			SourceFile NewStartingPoint = StartingPoints[LargestIdx].Key;
			if(NewStartingPoint != Files[0])
			{
				int Offset = Array.IndexOf(Files, NewStartingPoint);

				SourceFile[] NewFiles = new SourceFile[Files.Length];
				for(int Idx = 0; Idx < Files.Length; Idx++)
				{
					NewFiles[Idx] = Files[(Offset + Idx) % Files.Length];
				}
				Files = NewFiles;
			}
		}

		/// <summary>
		/// Checks whether an include cycle matches another include cycle, ignoring the start location.
		/// </summary>
		/// <param name="Other">The cycle to compare against</param>
		/// <returns>True if the cycles are identical, false otherwise</returns>
		public bool Matches(IncludeCycle Other)
		{
			if(Other.Files.Length != Files.Length)
			{
				return false;
			}

			int Offset = Array.IndexOf(Other.Files, Files[0]);
			if(Offset == -1)
			{
				return false;
			}

			for(int Idx = 0; Idx < Files.Length; Idx++)
			{
				if(Files[Idx] != Other.Files[(Offset + Idx) % Files.Length])
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Returns a hash code for the current cycle, independent of the cycle's starting point.
		/// </summary>
		/// <returns>Hash code for the cycle</returns>
		public override int GetHashCode()
		{
			int HashCode = 0;
			for(int Idx = 0; Idx < Files.Length; Idx++)
			{
				// Note: we deliberately do not mutate the hash code in any way that is not commutative if we start
				// at any point in the array. Summing doesn't provide a huge amount of entropy, but it's safe.
				HashCode += Files[Idx].Location.GetHashCode();
			}
			return HashCode;
		}

		/// <summary>
		/// Creates a human-readable representation of the cycle
		/// </summary>
		/// <returns>String representation of the cycle</returns>
		public override string ToString()
		{
			return String.Join(" -> ", Files.Concat(Files).Take(Files.Length + 1).Select(x => x.Location.GetFileName()));
		}
	}

	/// <summary>
	/// Functions to find include cycles
	/// </summary>
	static class IncludeCycles
	{
		/// <summary>
		/// Finds all include cycles from the given set of source files
		/// </summary>
		/// <param name="Files">Sequence of source files to check</param>
		/// <returns>List of cycles</returns>
		public static List<IncludeCycle> FindAll(IEnumerable<SourceFile> Files)
		{
			List<IncludeCycle> Cycles = new List<IncludeCycle>();
			foreach(SourceFile File in Files)
			{
				FindCyclesRecursive(File, new List<SourceFile>(), new HashSet<SourceFile>(), Cycles);
			}
			return Cycles;
		}

		/// <summary>
		/// Recurses through the given file and all of its includes, attempting to identify cycles.
		/// </summary>
		/// <param name="File">The file to search through</param>
		/// <param name="Includes">Current include stack of the preprocessor</param>
		/// <param name="VisitedFiles">Set of files that have already been checked for cycles</param>
		/// <param name="Cycles">List which receives any cycles that are found</param>
		static void FindCyclesRecursive(SourceFile File, List<SourceFile> Includes, HashSet<SourceFile> VisitedFiles, List<IncludeCycle> Cycles)
		{
			// Check if this include forms a cycle
			int IncludeIdx = Includes.IndexOf(File);
			if(IncludeIdx != -1)
			{
				IncludeCycle NewCycle = new IncludeCycle(Includes.Skip(IncludeIdx));
				for(int Idx = 0;;Idx++)
				{
					if(Idx == Cycles.Count)
					{
						Cycles.Add(NewCycle);
						break;
					}
					else if(Cycles[Idx].Matches(NewCycle))
					{
						Cycles[Idx].AddStartingPoint(NewCycle.Files[0]);
						break;
					}
				}
			}

			// If we haven't already looked from cycles from this include, search now
			if(!VisitedFiles.Contains(File))
			{
				VisitedFiles.Add(File);

				Includes.Add(File);
				foreach(PreprocessorMarkup Markup in File.Markup)
				{
					if(Markup.Type == PreprocessorMarkupType.Include && Markup.IncludedFile != null)
					{
						SourceFile IncludedFile = Markup.IncludedFile;
						if(!IncludedFile.Flags.HasFlag(SourceFileFlags.External))
						{
							FindCyclesRecursive(Markup.IncludedFile, Includes, VisitedFiles, Cycles);
						}
					}
				}
				Includes.RemoveAt(Includes.Count - 1);
			}
		}
	}
}
