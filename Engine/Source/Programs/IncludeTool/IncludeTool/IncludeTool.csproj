<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <TargetFramework>net6.0</TargetFramework>
    <Configuration Condition=" '$(Configuration)' == '' ">Development</Configuration>
    <OutputType>Exe</OutputType>
    <AppendTargetFrameworkToOutputPath>false</AppendTargetFrameworkToOutputPath>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <Configurations>Debug;Release;Development</Configurations>
    <AssemblyName>IncludeTool</AssemblyName>
    <RootNamespace>IncludeTool</RootNamespace>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages> <!-- remove non english resource languages -->
    <AppendRuntimeIdentifierToOutputPath>false</AppendRuntimeIdentifierToOutputPath>
    <IsWindows Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::Windows)))' == 'true'">true</IsWindows> 
    <IsOSX Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::OSX)))' == 'true'">true</IsOSX> 
    <IsLinux Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::Linux)))' == 'true'">true</IsLinux> 
    <DebugType>pdbonly</DebugType> 
    <Nullable>disable</Nullable>
  </PropertyGroup>
  <Choose>
    <When Condition="'$(IsOSX)'=='true'">
      <PropertyGroup>
        <!-- Set runtie identifier for osx so that it produces a app host executable (consistency with the other platforms) -->
        <RuntimeIdentifier>osx-x64</RuntimeIdentifier>
      </PropertyGroup>
    </When>
  </Choose>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Development|AnyCPU' ">
    <Optimize>true</Optimize>
    <OutputPath>..\..\..\..\Binaries\DotNET\IncludeTool</OutputPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|AnyCPU' ">
    <Optimize>true</Optimize>
    <OutputPath>..\..\..\..\Binaries\DotNET\IncludeTool</OutputPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <OutputPath>..\..\..\..\Binaries\DotNET\IncludeTool</OutputPath>
    <DebugSymbols>true</DebugSymbols>
    <DebugType>full</DebugType>
  </PropertyGroup>
  <ItemGroup>
    <PackageReference Include="Microsoft.CodeAnalysis.CSharp" Version="3.7.0" />
    <PackageReference Include="Microsoft.Win32.Registry" Version="5.0.0" />
    <PackageReference Include="System.Management" Version="4.7.0" />
    <PackageReference Include="System.Security.Cryptography.Csp" Version="4.3.0" />
    <PackageReference Include="System.ServiceProcess.ServiceController" Version="4.7.0" />

    <!-- Required by Ionic -->
    <PackageReference Include="System.Security.Permissions" Version="4.7.0" />
    <PackageReference Include="System.Text.Encoding.CodePages" Version="4.7.1" />

    <!-- Required for Hololens -->
    <PackageReference Include="System.Reflection.MetadataLoadContext" Version="4.7.2" /> 
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Build\EpicGames.Build.csproj" />
  </ItemGroup>
  <ItemGroup>
    <Reference Include="fastJSON">
      <HintPath>..\..\..\..\Binaries\ThirdParty\FastJson\netstandard2.0\FastJson.dll</HintPath>
    </Reference>
  </ItemGroup>
  <ItemGroup>
    <Compile Include="../../../../Platforms/*/Source/Programs/IncludeTool/**/*.cs">
      <!-- Since RecursiveDir is bugged on VS Mac, we use the FullPath, and use Regex to pull out portions of the path to get Platform name -->
      <Link>Platform/$([System.Text.RegularExpressions.Regex]::Replace(%(FullPath), '^(.*[\\/]Platforms[\\/])(.+?)[\\/](.*)$', '$2'))/%(Filename)%(Extension)</Link>
    </Compile>
    <Compile Include="../../../../Restricted/*/Platforms/*/Source/Programs/IncludeTool/**/*.cs">
      <!-- Restricted may or may not be in a Platforms dir, so just use the subdir directly, even tho it makes for a
       messier directory structure for these few files -->
      <Link>Platform/$([System.Text.RegularExpressions.Regex]::Replace(%(FullPath), '^(.*[\\/]Platforms[\\/])(.+?)[\\/](.*)$', '$2'))/%(Filename)%(Extension)</Link>
    </Compile>
    <Compile Include="../../../../Restricted/*/Source/Programs/IncludeTool/**/*.cs">
      <!-- Restricted may or may not be in a Platforms dir, so just use the subdir directly, even tho it makes for a
       messier directory structure for these few files -->
      <Link>$([System.Text.RegularExpressions.Regex]::Replace(%(FullPath), '^.+?[\\/]IncludeTool[\\/]', ''))</Link>
    </Compile>
  </ItemGroup>
</Project>