// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Win32.SafeHandles;
using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Possible priority values for new processes.
	/// </summary>
	enum ManagedProcessPriority
	{
		BelowNormal,
		Normal,
		AboveNormal,
	}

	/// <summary>
	/// Encapsulates a managed child process, from which we can read the console output.
	/// Uses job objects to ensure that the process will be terminated automatically by the O/S if the current process is terminated, and polls pipe reads to avoid blocking on unmanaged code
	/// if the calling thread is terminated. Currently only implemented for Windows; makes heavy use of P/Invoke.
	/// </summary>
	class ManagedProcess : IDisposable
	{
		[StructLayout(LayoutKind.Sequential)]
		class SECURITY_ATTRIBUTES
		{
			public int nLength;
			public IntPtr lpSecurityDescriptor;
			public int bInheritHandle;
		}

		[StructLayout(LayoutKind.Sequential)]
		class PROCESS_INFORMATION
		{
			public IntPtr hProcess;
			public IntPtr hThread;
			public uint dwProcessId;
			public uint dwThreadId;
		}

		[StructLayout(LayoutKind.Sequential)]
		class STARTUPINFO
		{
			public int cb;
			public string lpReserved;
			public string lpDesktop;
			public string lpTitle;
			public uint dwX;
			public uint dwY;
			public uint dwXSize;
			public uint dwYSize;
			public uint dwXCountChars;
			public uint dwYCountChars;
			public uint dwFillAttribute;
			public uint dwFlags;
			public short wShowWindow;
			public short cbReserved2;
			public IntPtr lpReserved2;
			public SafeHandle hStdInput;
			public SafeHandle hStdOutput;
			public SafeHandle hStdError;
		}

		const UInt32 JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE = 0x00002000;
		const UInt32 JOB_OBJECT_LIMIT_PROCESS_MEMORY = 0x00000100;

		[StructLayout(LayoutKind.Sequential)]
		struct JOBOBJECT_BASIC_LIMIT_INFORMATION
		{
			public Int64 PerProcessUserTimeLimit;
			public Int64 PerJobUserTimeLimit;
			public UInt32 LimitFlags;
			public UIntPtr MinimumWorkingSetSize;
			public UIntPtr MaximumWorkingSetSize;
			public UInt32 ActiveProcessLimit;
			public Int64 Affinity;
			public UInt32 PriorityClass;
			public UInt32 SchedulingClass;
		}

		[StructLayout(LayoutKind.Sequential)]
		struct IO_COUNTERS
		{
			public UInt64 ReadOperationCount;
			public UInt64 WriteOperationCount;
			public UInt64 OtherOperationCount;
			public UInt64 ReadTransferCount;
			public UInt64 WriteTransferCount;
			public UInt64 OtherTransferCount;
		}

		[StructLayout(LayoutKind.Sequential)]
		struct JOBOBJECT_EXTENDED_LIMIT_INFORMATION
		{
			public JOBOBJECT_BASIC_LIMIT_INFORMATION BasicLimitInformation;
			public IO_COUNTERS IoInfo;
			public UIntPtr ProcessMemoryLimit;
			public UIntPtr JobMemoryLimit;
			public UIntPtr PeakProcessMemoryUsed;
			public UIntPtr PeakJobMemoryUsed;
		}

		[DllImport("kernel32.dll", SetLastError=true)]
		static extern SafeFileHandle CreateJobObject(IntPtr SecurityAttributes, IntPtr Name);

		const int JobObjectExtendedLimitInformation = 9;

        [DllImport("kernel32.dll", SetLastError=true)]
        static extern int SetInformationJobObject(SafeFileHandle hJob, int JobObjectInfoClass, IntPtr lpJobObjectInfo, int cbJobObjectInfoLength);

		[DllImport("kernel32.dll", SetLastError=true)]
		static extern int AssignProcessToJobObject(SafeFileHandle hJob, IntPtr hProcess);

		[DllImport("kernel32.dll", SetLastError=true)]
		static extern int CloseHandle(IntPtr hObject);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int CreatePipe(out SafeFileHandle hReadPipe, out SafeFileHandle hWritePipe, SECURITY_ATTRIBUTES lpPipeAttributes, uint nSize);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int SetHandleInformation(SafeFileHandle hObject, int dwMask, int dwFlags);

		const int HANDLE_FLAG_INHERIT = 1;

        const int STARTF_USESTDHANDLES = 0x00000100;

		[Flags]
		enum ProcessCreationFlags : int
		{
			CREATE_NO_WINDOW = 0x08000000,
			CREATE_SUSPENDED = 0x00000004,
			CREATE_BREAKAWAY_FROM_JOB = 0x01000000,
			BELOW_NORMAL_PRIORITY_CLASS = 0x00004000,
			NORMAL_PRIORITY_CLASS = 0x00000020,
			ABOVE_NORMAL_PRIORITY_CLASS = 0x00008000,
		}

        [DllImport("kernel32.dll", SetLastError=true)]
        static extern int CreateProcess(/*[MarshalAs(UnmanagedType.LPTStr)]*/ string lpApplicationName, StringBuilder lpCommandLine, IntPtr lpProcessAttributes, IntPtr lpThreadAttributes, bool bInheritHandles, ProcessCreationFlags dwCreationFlags, IntPtr lpEnvironment, /*[MarshalAs(UnmanagedType.LPTStr)]*/ string lpCurrentDirectory, STARTUPINFO lpStartupInfo, PROCESS_INFORMATION lpProcessInformation);

		[DllImport("kernel32.dll", SetLastError=true)]
		static extern int ResumeThread(IntPtr hThread);

		[DllImport("kernel32.dll", SetLastError=true)]
		static extern IntPtr GetCurrentProcess();

		[DllImport("kernel32.dll", SetLastError=true)]
		static extern int TerminateProcess(SafeFileHandle hProcess, uint uExitCode);

		[DllImport("kernel32.dll", SetLastError=true)]
		static extern uint WaitForSingleObject(SafeFileHandle hHandle, uint dwMilliseconds);

		const uint INFINITE = 0xFFFFFFFF;

        const int DUPLICATE_SAME_ACCESS = 2;

        [DllImport("kernel32.dll", SetLastError=true)]    
        static extern int DuplicateHandle(
            IntPtr hSourceProcessHandle,
            SafeHandle hSourceHandle,
            IntPtr hTargetProcess,
            out SafeWaitHandle targetHandle,
            int dwDesiredAccess,
            bool bInheritHandle,
            int dwOptions
        );

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern int GetExitCodeProcess(SafeFileHandle hProcess, out int lpExitCode);

		/// <summary>
		/// Handle for the child process.
		/// </summary>
		SafeFileHandle ProcessHandle;

		/// <summary>
		/// The write end of the child process' stdin pipe.
		/// </summary>
		SafeFileHandle StdInWrite;

		/// <summary>
		/// The read end of the child process' stdout pipe.
		/// </summary>
		SafeFileHandle StdOutRead;

		/// <summary>
		/// Output stream for the child process.
		/// </summary>
		FileStream InnerStream;

		/// <summary>
		/// Reader for the process' output stream.
		/// </summary>
		StreamReader ReadStream;

		/// <summary>
		/// Handle to the native job object that this process is added to. This handle is closed by the Dispose() method (and will automatically be closed by the OS on process exit),
		/// resulting in the child process being killed.
		/// </summary>
		SafeFileHandle JobHandle;

		/// <summary>
		/// Spawns a new managed process.
		/// </summary>
		/// <param name="FileName">Path to the executable to be run</param>
		/// <param name="CommandLine">Command line arguments for the process</param>
		/// <param name="WorkingDirectory">Working directory for the new process. May be null to use the current working directory.</param>
		/// <param name="Environment">Environment variables for the new process. May be null, in which case the current process' environment is inherited</param>
		/// <param name="Input">Text to be passed via stdin to the new process. May be null.</param>
		public ManagedProcess(string FileName, string CommandLine, string WorkingDirectory, IReadOnlyDictionary<string, string> Environment, string Input, ManagedProcessPriority Priority, ulong MemoryLimit)
		{
			// Create the job object that the child process will be added to
			JobHandle = CreateJobObject(IntPtr.Zero, IntPtr.Zero);
			if(JobHandle == null)
			{
				throw new Win32Exception();
			}

			// Configure the job object to terminate the processes added to it when the handle is closed
			JOBOBJECT_EXTENDED_LIMIT_INFORMATION LimitInformation = new JOBOBJECT_EXTENDED_LIMIT_INFORMATION();
			LimitInformation.BasicLimitInformation.LimitFlags = JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE | JOB_OBJECT_LIMIT_PROCESS_MEMORY;
			LimitInformation.ProcessMemoryLimit = new UIntPtr(MemoryLimit);

			int Length = Marshal.SizeOf(typeof(JOBOBJECT_EXTENDED_LIMIT_INFORMATION));
			IntPtr LimitInformationPtr = Marshal.AllocHGlobal(Length);
			Marshal.StructureToPtr(LimitInformation, LimitInformationPtr, false);

			if(SetInformationJobObject(JobHandle, JobObjectExtendedLimitInformation, LimitInformationPtr, Length) == 0)
			{
				throw new Win32Exception();
			}

			// Create the child process
			IntPtr EnvironmentBlock = IntPtr.Zero;
			SafeFileHandle StdInRead = null;
			SafeFileHandle StdOutWrite = null;
			SafeWaitHandle StdErrWrite = null;
			try
			{
				// Create stdin and stdout pipes for the child process. We'll close the handles for the child process' ends after it's been created.
				SECURITY_ATTRIBUTES SecurityAttributes = new SECURITY_ATTRIBUTES();
				SecurityAttributes.bInheritHandle = 1;

				if(CreatePipe(out StdInRead, out StdInWrite, SecurityAttributes, 0) == 0 || SetHandleInformation(StdInWrite, HANDLE_FLAG_INHERIT, 0) == 0)
				{
					throw new Win32Exception();
				}
				if(CreatePipe(out StdOutRead, out StdOutWrite, SecurityAttributes, 0) == 0 || SetHandleInformation(StdOutRead, HANDLE_FLAG_INHERIT, 0) == 0)
				{
					throw new Win32Exception();
				}
				if(DuplicateHandle(GetCurrentProcess(), StdOutWrite, GetCurrentProcess(), out StdErrWrite, DUPLICATE_SAME_ACCESS, true, 0) == 0)
				{
					throw new Win32Exception();
				}

				// Create the environment block for the child process, if necessary.
				if(Environment != null)
				{
					// The native format for the environment block is a sequence of null terminated strings with a final null terminator.
					List<byte> EnvironmentBytes = new List<byte>();
					foreach(KeyValuePair<string, string> Pair in Environment)
					{
						EnvironmentBytes.AddRange(Encoding.UTF8.GetBytes(Pair.Key));
						EnvironmentBytes.Add((byte)'=');
						EnvironmentBytes.AddRange(Encoding.UTF8.GetBytes(Pair.Value));
						EnvironmentBytes.Add((byte)0);
					}
					EnvironmentBytes.Add((byte)0);

					// Allocate an unmanaged block of memory to store it.
					EnvironmentBlock = Marshal.AllocHGlobal(EnvironmentBytes.Count);
					Marshal.Copy(EnvironmentBytes.ToArray(), 0, EnvironmentBlock, EnvironmentBytes.Count);
				}

				// Set the startup parameters for the new process
				STARTUPINFO StartupInfo = new STARTUPINFO();
                StartupInfo.cb = Marshal.SizeOf(StartupInfo);
				StartupInfo.hStdInput = StdInRead;
				StartupInfo.hStdOutput = StdOutWrite;
				StartupInfo.hStdError = StdErrWrite;
				StartupInfo.dwFlags = STARTF_USESTDHANDLES;

				PROCESS_INFORMATION ProcessInfo = new PROCESS_INFORMATION();
				try
				{
					// Get the flags to create the new process
					ProcessCreationFlags Flags = ProcessCreationFlags.CREATE_NO_WINDOW | ProcessCreationFlags.CREATE_SUSPENDED | ProcessCreationFlags.CREATE_BREAKAWAY_FROM_JOB;
					switch(Priority)
					{
						case ManagedProcessPriority.BelowNormal:
							Flags |= ProcessCreationFlags.BELOW_NORMAL_PRIORITY_CLASS;
							break;
						case ManagedProcessPriority.Normal:
							Flags |= ProcessCreationFlags.NORMAL_PRIORITY_CLASS;
							break;
						case ManagedProcessPriority.AboveNormal:
							Flags |= ProcessCreationFlags.ABOVE_NORMAL_PRIORITY_CLASS;
							break;
					}

					// Create the new process as suspended, so we can modify it before it starts executing (and potentially preempting us)
					if(CreateProcess(null, new StringBuilder("\"" + FileName + "\" " + CommandLine), IntPtr.Zero, IntPtr.Zero, true, Flags, EnvironmentBlock, WorkingDirectory, StartupInfo, ProcessInfo) == 0)
					{
						throw new Win32Exception();
					}

					// Add it to our job object
					if(AssignProcessToJobObject(JobHandle, ProcessInfo.hProcess) == 0)
					{
						throw new Win32Exception();
					}

					// Allow the thread to start running
					if(ResumeThread(ProcessInfo.hThread) == -1)
					{
						throw new Win32Exception();
					}

					// If we have any input text, write it to stdin now
					using(StreamWriter StdInWriter = new StreamWriter(new FileStream(StdInWrite, FileAccess.Write, 4096, false), Console.InputEncoding, 4096))
					{
						if(!String.IsNullOrEmpty(Input))
						{
							StdInWriter.WriteLine(Input);
							StdInWriter.Flush();
						}
					}

					// Create the stream objects for reading the process output
					InnerStream = new FileStream(StdOutRead, FileAccess.Read, 4096, false);
					ReadStream = new StreamReader(InnerStream, Console.OutputEncoding);

					// Wrap the process handle in a SafeFileHandle
					ProcessHandle = new SafeFileHandle(ProcessInfo.hProcess, true);
				}
				finally
				{
					if(ProcessInfo.hProcess != IntPtr.Zero && ProcessHandle == null)
					{
						CloseHandle(ProcessInfo.hProcess);
					}
					if(ProcessInfo.hThread != IntPtr.Zero)
					{
						CloseHandle(ProcessInfo.hThread);
					}
				}
			}
			finally
			{
				if(EnvironmentBlock != IntPtr.Zero)
				{
					Marshal.FreeHGlobal(EnvironmentBlock);
					EnvironmentBlock = IntPtr.Zero;
				}
				if(StdInRead != null)
				{
					StdInRead.Dispose();
					StdInRead = null;
				}
				if(StdOutWrite != null)
				{
					StdOutWrite.Dispose();
					StdOutWrite = null;
				}
				if(StdErrWrite != null)
				{
					StdErrWrite.Dispose();
					StdErrWrite = null;
				}
			}
		}

		/// <summary>
		/// Free the managed resources for this process
		/// </summary>
		public void Dispose()
		{
			if(ProcessHandle != null)
			{
				ProcessHandle.Dispose();
				ProcessHandle = null;
			}
			if(StdInWrite != null)
			{
				StdInWrite.Dispose();
				StdInWrite = null;
			}
			if(StdOutRead != null)
			{
				StdOutRead.Dispose();
				StdOutRead = null;
			}
			if(JobHandle != null)
			{
				JobHandle.Dispose();
				JobHandle = null;
			}
		}

		/// <summary>
		/// Read all the output from the process. Does not return until the process terminates.
		/// </summary>
		/// <returns>List of output lines</returns>
		public bool TryReadAllLines(TimeSpan TimeLimit, List<string> OutputLines)
		{
			// Capture the start time
			DateTime StartTime = DateTime.UtcNow;

			// Manually read all the output lines from the stream. Using ReadToEndAsync() or ReadAsync() on the StreamReader is abysmally slow, especially
			// for high-volume processes. Manually picking out the lines via a buffered ReadAsync() call was found to be 6x faster on 'p4 -ztag have' calls.
			byte[] Buffer = new byte[32 * 1024];
			byte LastCharacter = 0;
			int NumBytesInBuffer = 0;
			while((DateTime.UtcNow - StartTime) < TimeLimit)
			{
				// Fill the buffer, reentering managed code every 20ms to allow thread abort exceptions to be thrown
				Task<int> ReadTask = InnerStream.ReadAsync(Buffer, NumBytesInBuffer, Buffer.Length - NumBytesInBuffer);
				while(!ReadTask.Wait(20))
				{
				}

				// Update the buffer size
				if(ReadTask.Result == 0)
				{
					return true;
				}

				// Otherwise append to the existing buffer
				NumBytesInBuffer += ReadTask.Result;

				// Pull out all the complete output lines
				int LastStartIdx = 0;
				for(int Idx = 0; Idx < NumBytesInBuffer; Idx++)
				{
					if(Buffer[Idx] == '\r' || Buffer[Idx] == '\n')
					{
						if(Buffer[Idx] != '\n' || LastCharacter != '\r')
						{
							OutputLines.Add(Encoding.UTF8.GetString(Buffer, LastStartIdx, Idx - LastStartIdx));
						}
						LastStartIdx = Idx + 1;
					}
					LastCharacter = Buffer[Idx];
				}

				// Shuffle everything back to the start of the buffer
				Array.Copy(Buffer, LastStartIdx, Buffer, 0, Buffer.Length - LastStartIdx);
				NumBytesInBuffer -= LastStartIdx;
			}
			return false;
		}

		/// <summary>
		/// Block until the process outputs a line of text, or terminates.
		/// </summary>
		/// <param name="Line">Variable to receive the output line</param>
		/// <returns>True if a line was read, false if the process terminated without writing another line.</returns>
		public bool TryReadLine(out string Line)
		{
			// Busy wait for the ReadLine call to finish, so we can get interrupted by thread abort exceptions
			Task<string> ReadLineTask = ReadStream.ReadLineAsync();
			for(;;)
			{
				if(ReadLineTask.Wait(20))
				{
					Line = ReadLineTask.IsCompleted? ReadLineTask.Result : null;
					return Line != null;
				}
			}
		}

		/// <summary>
		/// Terminate the process and waits for it to exit
		/// </summary>
		/// <param name="ExitCode">The process exit code</param>
		public void Terminate(uint ExitCode)
		{
			TerminateProcess(ProcessHandle, ExitCode);
			WaitForSingleObject(ProcessHandle, INFINITE);
		}

		/// <summary>
		/// The exit code of the process. Throws an exception if the process has not terminated.
		/// </summary>
		public int ExitCode
		{
			get
			{
				int Value;
				if(GetExitCodeProcess(ProcessHandle, out Value) == 0)
				{
					throw new Win32Exception();
				}
				return Value;
			}
		}

		/// <summary>
		/// Gets the native flag for the given priority class, as passed 
		/// </summary>
		/// <param name="Priority">The priority value</param>
		/// <returns>The native priority class flag</returns>
		private static int GetNativePriorityClassFlag(ManagedProcessPriority Priority)
		{
			// Values below are taken from documentation on MSDN
			switch(Priority)
			{
				case ManagedProcessPriority.BelowNormal:
					// BELOW_NORMAL_PRIORITY_CLASS
					return 0x4000;
				case ManagedProcessPriority.Normal:
					// NORMAL_PRIORITY_CLASS
					return 0x0020;
				case ManagedProcessPriority.AboveNormal:
					// ABOVE_NORMAL_PRIORITY_CLASS
					return 0x8000;
			}
			throw new Exception(String.Format("Invalid priority value: {0}", Priority));
		}
	}
}
