// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Reference to an output file, in the context of an include stack
	/// </summary>
	class OutputFileReference
	{
		/// <summary>
		/// The file to include
		/// </summary>
		public readonly OutputFile File;

		/// <summary>
		/// Fragments which it uniquely provides, which have not been included by a previous file.
		/// </summary>
		public readonly SourceFragment[] UniqueFragments;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="File">The file being included</param>
		/// <param name="UniqueFragments">Unique fragments supplied by this file</param>
		public OutputFileReference(OutputFile File, IEnumerable<SourceFragment> UniqueFragments)
		{
			this.File = File;
			this.UniqueFragments = UniqueFragments.ToArray();
		}

		/// <summary>
		/// Convert to a string for the debugger
		/// </summary>
		/// <returns>Str</returns>
		public override string ToString()
		{
			return File.ToString();
		}
	}

	/// <summary>
	/// An include directive from an output file
	/// </summary>
	class OutputFileInclude
	{
		/// <summary>
		/// The markup index in the source file
		/// </summary>
		public readonly int MarkupIdx;

		/// <summary>
		/// The file being included
		/// </summary>
		public readonly OutputFile TargetFile;

		/// <summary>
		/// The expanded list of unique output files matching the equivalent input file, which have not already been included by the file containing this #include directive
		/// </summary>
		public List<OutputFileReference> ExpandedReferences;

		/// <summary>
		/// The expanded list of output files matching the equivalent input file
		/// </summary>
		public List<OutputFile> FinalFiles = new List<OutputFile>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="MarkupIdx">Index into the PreprocessorMarkup array for the file containing this include</param>
		/// <param name="TargetFile">The file being included</param>
		/// <param name="IsSeparable">Whether this include can be removed from the file</param>
		public OutputFileInclude(int MarkupIdx, OutputFile TargetFile)
		{
			this.MarkupIdx = MarkupIdx;
			this.TargetFile = TargetFile;
		}

		/// <summary>
		/// Converts this object to a string for the debugger
		/// </summary>
		/// <returns>String representation of this object</returns>
		public override string ToString()
		{
			return String.Format("{0}: {1}", MarkupIdx, TargetFile);
		}
	}

	/// <summary>
	/// Represents an optimized output file
	/// </summary>
	class OutputFile
	{
		/// <summary>
		/// The SourceFile that corresponds to this output file
		/// </summary>
		public readonly SourceFile InputFile;

		/// <summary>
		/// The other output files included from this file
		/// </summary>
		public readonly List<OutputFileInclude> Includes;

		/// <summary>
		/// Fragments which this output file has dependencies on
		/// </summary>
		public HashSet<SourceFragment> Dependencies;

		/// <summary>
		/// Symbols which need explicit forward declarations
		/// </summary>
		public List<Symbol> ForwardDeclarations;

		/// <summary>
		/// Fragments which are included through this file
		/// </summary>
		public HashSet<SourceFragment> IncludedFragments;

		/// <summary>
		/// All the files which are included by this file
		/// </summary>
		public HashList<OutputFile> IncludedFiles;

		/// <summary>
		/// All the files which were originally included by this file
		/// </summary>
		public HashList<OutputFile> OriginalIncludedFiles;

		/// <summary>
		/// Constructor
		/// </summary>
		public OutputFile(SourceFile InputFile, List<OutputFileInclude> Includes, HashSet<SourceFragment> Dependencies, List<Symbol> ForwardDeclarations)
		{
			this.InputFile = InputFile;
			this.Includes = Includes;
			this.Dependencies = Dependencies;
			this.ForwardDeclarations = ForwardDeclarations;

			// Build the list of satisfied dependencies
			IncludedFragments = new HashSet<SourceFragment>();
			IncludedFragments.UnionWith(Includes.SelectMany(x => x.FinalFiles).SelectMany(x => x.IncludedFragments));
			IncludedFragments.UnionWith(InputFile.Fragments);

			// Build the list of all the included files, so other output files that include it can expand it out.
			IncludedFiles = new HashList<OutputFile>();
			IncludedFiles.UnionWith(Includes.SelectMany(x => x.FinalFiles).SelectMany(x => x.IncludedFiles));
			IncludedFiles.Add(this);

			// Build the list of all the originally included files, so that we can make files that include it standalone
			OriginalIncludedFiles = new HashList<OutputFile>();
			OriginalIncludedFiles.UnionWith(Includes.Select(x => x.TargetFile).Where(x => x != null).SelectMany(x => x.OriginalIncludedFiles));
			OriginalIncludedFiles.Add(this);
		}

		/// <summary>
		/// Convert the object to a string for debugging purposes
		/// </summary>
		/// <returns>String representation of the object</returns>
		public override string ToString()
		{
			return InputFile.ToString();
		}
	}

	/// <summary>
	/// Class to construct output files from optimized input files
	/// </summary>
	static class OutputFileBuilder
	{
		/// <summary>
		/// Create optimized output files from the given input files
		/// </summary>
		/// <param name="CppFiles">Input files to optimize</param>
		/// <param name="Log">Writer for log messages</param>
		/// <returns>Array of output files</returns>
		public static void PrepareFilesForOutput(IEnumerable<SourceFile> CppFiles, Dictionary<SourceFile, SourceFile> CppFileToHeaderFile, Dictionary<Symbol, SourceFile> FwdSymbolToInputHeader, bool bMakeStandalone, bool bUseOriginalIncludes, LineBasedTextWriter Log)
		{
			// Cache of all the created output files
			Dictionary<SourceFile, OutputFile> OutputFileLookup = new Dictionary<SourceFile,OutputFile>();

			// Create output files for all the forward declaration headers
			Dictionary<Symbol, OutputFile> FwdSymbolToHeader = new Dictionary<Symbol, OutputFile>();
			foreach(KeyValuePair<Symbol, SourceFile> Pair in FwdSymbolToInputHeader)
			{
				List<SourceFile> InputFileStack = new List<SourceFile>();
				InputFileStack.Add(Pair.Value);

				HashList<OutputFile> IncludedFiles = new HashList<OutputFile>();
				FwdSymbolToHeader[Pair.Key] = FindOrCreateOutputFile(InputFileStack, CppFileToHeaderFile, IncludedFiles, OutputFileLookup, FwdSymbolToHeader, bMakeStandalone, bUseOriginalIncludes, Log);
			}

			// Create all the placeholder output files 
			foreach(SourceFile CppFile in CppFiles)
			{
				List<SourceFile> InputFileStack = new List<SourceFile>();
				InputFileStack.Add(CppFile);

				HashList<OutputFile> IncludedFiles = new HashList<OutputFile>();
				FindOrCreateOutputFile(InputFileStack, CppFileToHeaderFile, IncludedFiles, OutputFileLookup, FwdSymbolToHeader, bMakeStandalone, bUseOriginalIncludes, Log);
			}

			// Set the results on the source files
			foreach(SourceFile File in OutputFileLookup.Keys)
			{
				OutputFile OutputFile = OutputFileLookup[File];
				foreach(OutputFileInclude Include in OutputFile.Includes)
				{
					if(Include.MarkupIdx < 0)
					{
						File.MissingIncludes.AddRange(Include.FinalFiles.Select(x => x.InputFile));
					}
					else
					{
						File.Markup[Include.MarkupIdx].OutputIncludedFiles = Include.FinalFiles.Select(x => x.InputFile).ToList();
					}
				}
				foreach(Symbol Symbol in OutputFile.ForwardDeclarations)
				{
					if(!String.IsNullOrEmpty(Symbol.ForwardDeclaration))
					{
						File.ForwardDeclarations.Add(Symbol.ForwardDeclaration);
					}
				}
			}
		}

		/// <summary>
		/// Find or create an output file for a corresponding input file
		/// </summary>
		/// <param name="InputFile">The input file</param>
		/// <param name="IncludeStack">The current include stack</param>
		/// <param name="OutputFiles">List of output files</param>
		/// <param name="OutputFileLookup">Mapping from source file to output file</param>
		/// <returns>The new or existing output file</returns>
		static OutputFile FindOrCreateOutputFile(List<SourceFile> InputFileStack, Dictionary<SourceFile, SourceFile> CppFileToHeaderFile, HashList<OutputFile> PreviousFiles, Dictionary<SourceFile, OutputFile> OutputFileLookup, Dictionary<Symbol, OutputFile> FwdSymbolToHeader, bool bMakeStandalone, bool bUseOriginalIncludes, LineBasedTextWriter Log)
		{
			// Get the file at the top of the stack
			SourceFile InputFile = InputFileStack[InputFileStack.Count - 1];

			// Try to find an existing file
			OutputFile OutputFile;
			if(OutputFileLookup.TryGetValue(InputFile, out OutputFile))
			{
				if(OutputFile == null)
				{
					throw new Exception("Circular include dependencies are not allowed.");
				}
				foreach(OutputFile IncludedFile in OutputFile.OriginalIncludedFiles.Where(x => !PreviousFiles.Contains(x)))
				{
					PreviousFiles.Add(IncludedFile);
				}
			}
			else
			{
				// Add a placeholder entry in the output file lookup, so we can detect circular include dependencies
				OutputFileLookup[InputFile] = null;

				// Duplicate the list of previously included files, so we can construct the 
				List<OutputFile> PreviousFilesCopy = new List<OutputFile>(PreviousFiles);

				// Build a list of includes for this file. First include is a placeholder for any missing includes that need to be inserted.
				List<OutputFileInclude> Includes = new List<OutputFileInclude>();
				if((InputFile.Flags & SourceFileFlags.External) == 0)
				{
					for(int MarkupIdx = 0; MarkupIdx < InputFile.Markup.Length; MarkupIdx++)
					{
						PreprocessorMarkup Markup = InputFile.Markup[MarkupIdx];
						if(Markup.IsActive && Markup.IncludedFile != null && (Markup.IncludedFile.Flags & SourceFileFlags.Inline) == 0 && Markup.IncludedFile.Counterpart == null)
						{
							InputFileStack.Add(Markup.IncludedFile);
							OutputFile IncludeFile = FindOrCreateOutputFile(InputFileStack, CppFileToHeaderFile, PreviousFiles, OutputFileLookup, FwdSymbolToHeader, bMakeStandalone, bUseOriginalIncludes, Log);
							InputFileStack.RemoveAt(InputFileStack.Count - 1);
							Includes.Add(new OutputFileInclude(MarkupIdx, IncludeFile));
						}
					}
				}

				// Find the matching header file
				OutputFile HeaderFile = null;
				if((InputFile.Flags & SourceFileFlags.TranslationUnit) != 0)
				{
					SourceFile CandidateHeaderFile;
					if(CppFileToHeaderFile.TryGetValue(InputFile, out CandidateHeaderFile) && (CandidateHeaderFile.Flags & SourceFileFlags.Standalone) != 0)
					{
						OutputFileLookup.TryGetValue(CandidateHeaderFile, out HeaderFile);
					}
				}

				// Create the output file.
				if((InputFile.Flags & SourceFileFlags.Output) != 0 && !bUseOriginalIncludes)
				{
					OutputFile = CreateOptimizedOutputFile(InputFile, HeaderFile, PreviousFilesCopy, Includes, InputFileStack, FwdSymbolToHeader, bMakeStandalone, Log);
				}
				else
				{
					OutputFile = CreatePassthroughOutputFile(InputFile, Includes, Log);
				}

				// Replace the null entry in the output file lookup that we added earlier
				OutputFileLookup[InputFile] = OutputFile;

				// Add this file to the list of included files
				PreviousFiles.Add(OutputFile);

				// If the output file dependends on something on the stack, make sure it's marked as pinned
				if((InputFile.Flags & SourceFileFlags.Pinned) == 0)
				{
					SourceFragment Dependency = OutputFile.Dependencies.FirstOrDefault(x => InputFileStack.Contains(x.File) && x.File != InputFile);
					if(Dependency != null)
					{
						throw new Exception(String.Format("'{0}' is not marked as pinned, but depends on '{1}' which includes it", InputFile.Location.GetFileName(), Dependency.UniqueName));
					}
				}
			}
			return OutputFile;
		}

		/// <summary>
		/// Creates an optimized output file
		/// </summary>
		/// <param name="InputFile">The input file that this output file corresponds to</param>
		/// <param name="HeaderFile">The corresponding header file</param>
		/// <param name="PreviousFiles">List of files parsed before this one</param>
		/// <param name="Includes">The active set of includes parsed for this file</param>
		/// <param name="InputFileStack">The active include stack</param>
		/// <param name="FwdSymbolToHeader"></param>
		/// <param name="bMakeStandalone">Whether to make this output file standalone</param>
		/// <param name="Log">Writer for log messages</param>
		public static OutputFile CreateOptimizedOutputFile(SourceFile InputFile, OutputFile HeaderFile, List<OutputFile> PreviousFiles, List<OutputFileInclude> Includes, List<SourceFile> InputFileStack, Dictionary<Symbol, OutputFile> FwdSymbolToHeader, bool bMakeStandalone, LineBasedTextWriter Log)
		{
			Debug.Assert(HeaderFile == null || (InputFile.Flags & SourceFileFlags.TranslationUnit) != 0);

			// Write the state
			InputFile.LogVerbose("InputFile={0}", InputFile.Location.FullName);
			InputFile.LogVerbose("InputFile.Flags={0}", InputFile.Flags.ToString());
			if(HeaderFile != null)
			{
				InputFile.LogVerbose("HeaderFile={0}", HeaderFile.InputFile.Location.FullName);
				InputFile.LogVerbose("HeaderFile.Flags={0}", HeaderFile.InputFile.Flags.ToString());
			}
			InputFile.LogVerbose("");
			for(int Idx = 0; Idx < InputFileStack.Count; Idx++)
			{
				InputFile.LogVerbose("InputFileStack[{0}]={1}", Idx, InputFileStack[Idx].Location.FullName);
			}
			InputFile.LogVerbose("");
			for(int Idx = 0; Idx < PreviousFiles.Count; Idx++)
			{
				InputFile.LogVerbose("PreviousFiles[{0}]={1}", Idx, PreviousFiles[Idx].InputFile.Location.FullName);
			}
			InputFile.LogVerbose("");
			for(int Idx = 0; Idx < Includes.Count; Idx++)
			{
			}
			InputFile.LogVerbose("");
			for(int Idx = 0; Idx < Includes.Count; Idx++)
			{
				OutputFileInclude Include = Includes[Idx];
				InputFile.LogVerbose("Includes[{0}]={1}", Idx, Includes[Idx].TargetFile.InputFile.Location.FullName);
				foreach(SourceFragment Fragment in Include.FinalFiles.SelectMany(x => x.IncludedFragments))
				{
					InputFile.LogVerbose("Includes[{0}].FinalFiles.IncludedFragments={1}", Idx, Fragment);
				}
			}

			// Traverse through all the included headers, figuring out the first unique include for each file and fragment
			HashSet<OutputFile> VisitedFiles = new HashSet<OutputFile>();
			HashSet<SourceFragment> VisitedFragments = new HashSet<SourceFragment>();

			// Go through the standalone headers first
			OutputFile MonolithicHeader = null;
			if(HeaderFile == null && (InputFile.Flags & SourceFileFlags.Standalone) != 0 && (InputFile.Flags & SourceFileFlags.External) == 0 && (InputFile.Flags & SourceFileFlags.Aggregate) == 0)
			{
				// Insert a dummy include to receive all the inserted headers
				OutputFileInclude ImplicitInclude = new OutputFileInclude(-1, null);
				ImplicitInclude.ExpandedReferences = new List<OutputFileReference>();
				Includes.Insert(0, ImplicitInclude);

				// Determine which monolithic header to use
				IEnumerable<OutputFile> PotentialMonolithicHeaders = PreviousFiles.Union(Includes.Select(x => x.TargetFile).Where(x => x != null).SelectMany(x => x.IncludedFiles));
				if(InputFile.Module != null && InputFile.Module.PublicDependencyModules.Union(InputFile.Module.PrivateDependencyModules).Any(x => x.Name == "Core"))
				{
					MonolithicHeader = PotentialMonolithicHeaders.FirstOrDefault(x => (x.InputFile.Flags & SourceFileFlags.IsCoreMinimal) != 0);
				}
				else
				{
					MonolithicHeader = PotentialMonolithicHeaders.FirstOrDefault(x => (x.InputFile.Flags & SourceFileFlags.IsCoreTypes) != 0);
				}

				// Update the dependencies to treat all the contents of a monolithic header as pinned
				if (MonolithicHeader != null)
				{
					SourceFragment[] UniqueFragments = MonolithicHeader.IncludedFragments.Except(VisitedFragments).ToArray();
					ImplicitInclude.ExpandedReferences.Add(new OutputFileReference(MonolithicHeader, UniqueFragments));
					VisitedFragments.UnionWith(UniqueFragments);
					VisitedFiles.Add(MonolithicHeader);
				}

				// Insert all the forward declaration headers, but only treat them as supplying the forward declarations themselves. They may happen to include
				// some utility classes (eg. TSharedPtr), and we don't want to include an unrelated header to satisfy that dependency.
				foreach(OutputFile FwdHeader in FwdSymbolToHeader.Values)
				{
					FindExpandedReferences(FwdHeader, ImplicitInclude.ExpandedReferences, VisitedFiles, VisitedFragments, true);
				}

				// Add all the other files
				if(bMakeStandalone)
				{
					foreach (OutputFile PreviousFile in PreviousFiles)
					{
						if((InputFile.Flags & SourceFileFlags.Standalone) != 0 && (PreviousFile.InputFile.Flags & SourceFileFlags.Inline) == 0 && (PreviousFile.InputFile.Flags & SourceFileFlags.Pinned) == 0 && VisitedFiles.Add(PreviousFile))
						{
							SourceFragment[] UniqueFragments = PreviousFile.IncludedFragments.Except(VisitedFragments).ToArray();
							ImplicitInclude.ExpandedReferences.Add(new OutputFileReference(PreviousFile, UniqueFragments));
							VisitedFragments.UnionWith(UniqueFragments);
						}
					}
				}
			}

			// Figure out a list of files which are uniquely reachable through each include. Force an include of the matching header as the first thing.
			OutputFileReference ForcedHeaderFileReference = null;
			foreach(OutputFileInclude Include in Includes)
			{
				if(Include.ExpandedReferences == null)
				{
					Include.ExpandedReferences = new List<OutputFileReference>();
					if(Include == Includes[0] && HeaderFile != null)
					{
						ForcedHeaderFileReference = new OutputFileReference(HeaderFile, HeaderFile.IncludedFragments);
						Include.ExpandedReferences.Add(ForcedHeaderFileReference);
						VisitedFragments.UnionWith(HeaderFile.IncludedFragments);
					}
					FindExpandedReferences(Include.TargetFile, Include.ExpandedReferences, VisitedFiles, VisitedFragments, true);
				}
			}

			// Find all the symbols which are referenced by this file
			HashSet<SourceFragment> FragmentsWithReferencedSymbols = new HashSet<SourceFragment>();
			foreach(SourceFragment Fragment in InputFile.Fragments)
			{
				foreach(KeyValuePair<Symbol, SymbolReferenceType> ReferencedSymbol in Fragment.ReferencedSymbols)
				{
					if(ReferencedSymbol.Value == SymbolReferenceType.RequiresDefinition)
					{
						FragmentsWithReferencedSymbols.Add(ReferencedSymbol.Key.Fragment);
					}
				}
			}

			// Aggregate headers are designed to explicitly include headers from the current module. Expand out a list of them, so they can be included when encountered.
			HashSet<OutputFile> ExplicitIncludes = new HashSet<OutputFile>();
			if((InputFile.Flags & SourceFileFlags.Aggregate) != 0)
			{
				foreach(OutputFileInclude Include in Includes)
				{
					ExplicitIncludes.UnionWith(Include.ExpandedReferences.Where(x => x.File.InputFile.Location.IsUnderDirectory(InputFile.Location.Directory)).Select(x => x.File));
				}
				foreach(OutputFileInclude Include in Includes)
				{
					ExplicitIncludes.Remove(Include.TargetFile);
				}
			}

			// Create the list of remaining dependencies for this file, and add any forward declarations
			HashSet<SourceFragment> Dependencies = new HashSet<SourceFragment>();
			List<Symbol> ForwardDeclarations = new List<Symbol>();
			AddForwardDeclarations(InputFile, ForwardDeclarations, Dependencies, FwdSymbolToHeader);

			// Reduce the list of includes to those that are required.
			for(int FragmentIdx = InputFile.Fragments.Length - 1, IncludeIdx = Includes.Count - 1; FragmentIdx >= 0; FragmentIdx--)
			{
				// Update the dependency lists for this fragment
				SourceFragment InputFragment = InputFile.Fragments[FragmentIdx];
				if(InputFragment.Dependencies != null)
				{
					Dependencies.UnionWith(InputFragment.Dependencies);
				}
				Dependencies.Remove(InputFragment);

				// Scan backwards through the list of includes, expanding each include to those which are required
				int MarkupMin = (FragmentIdx == 0)? -1 : InputFragment.MarkupMin;
				for(; IncludeIdx >= 0 && Includes[IncludeIdx].MarkupIdx >= MarkupMin; IncludeIdx--)
				{
					OutputFileInclude Include = Includes[IncludeIdx];

					// Always include the same header for aggregates
					if((InputFile.Flags & SourceFileFlags.Aggregate) != 0)
					{
						Include.FinalFiles.Insert(0, Include.TargetFile);
						Dependencies.ExceptWith(Include.TargetFile.IncludedFragments);
						Dependencies.UnionWith(Include.TargetFile.Dependencies);
					}

					// Include any indirectly included files
					for(int Idx = Include.ExpandedReferences.Count - 1; Idx >= 0; Idx--)
					{
						// Make sure we haven't already added it above
						OutputFileReference Reference = Include.ExpandedReferences[Idx];
						if(!Include.FinalFiles.Contains(Reference.File))
						{
							if(Dependencies.Any(x => Reference.UniqueFragments.Contains(x)) 
								|| (Reference.File.InputFile.Flags & SourceFileFlags.Pinned) != 0 
								|| Reference == ForcedHeaderFileReference
								|| Reference.File == MonolithicHeader
								|| ExplicitIncludes.Contains(Reference.File)
								|| ((InputFile.Flags & SourceFileFlags.Aggregate) != 0 && Reference.File == Include.TargetFile) // Always include the original header for aggregates. They are written explicitly to include certain files.
								|| Reference.UniqueFragments.Any(x => FragmentsWithReferencedSymbols.Contains(x)))
							{
								Include.FinalFiles.Insert(0, Reference.File);
								Dependencies.ExceptWith(Reference.File.IncludedFragments);
								Dependencies.UnionWith(Reference.File.Dependencies);
							}
						}
					}
				}
			}

			// Remove any includes that are already included by the matching header
			if(HeaderFile != null)
			{
				HashSet<OutputFile> HeaderIncludedFiles = new HashSet<OutputFile>(HeaderFile.Includes.SelectMany(x => x.FinalFiles));
				foreach(OutputFileInclude Include in Includes)
				{
					Include.FinalFiles.RemoveAll(x => HeaderIncludedFiles.Contains(x));
				}
			}

			// Check that all the dependencies have been satisfied
			if(Dependencies.Count > 0)
			{
				// Find those which are completely invalid
				List<SourceFragment> InvalidDependencies = Dependencies.Where(x => !InputFileStack.Contains(x.File)).ToList();
				if(InvalidDependencies.Count > 0)
				{
					Log.WriteLine("warning: {0} does not include {1}{2}; may have missing dependencies.", InputFile, String.Join(", ", InvalidDependencies.Select(x => x.Location.FullName).Take(3)), (InvalidDependencies.Count > 3)? String.Format(" and {0} others", InvalidDependencies.Count - 3) : "");
				}
				Dependencies.ExceptWith(InvalidDependencies);

				// Otherwise warn about those which were not pinned
				foreach(SourceFile DependencyFile in Dependencies.Select(x => x.File))
				{
					Log.WriteLine("warning: {0} is included by {1} ({2}), but depends on it and should be marked as pinned.", InputFile, DependencyFile, String.Join(" -> ", InputFileStack.SkipWhile(x => x != DependencyFile).Select(x => x.Location.GetFileName())));
				}

				// Mark it as non-standalone and pinned
				InputFile.Flags = (InputFile.Flags | SourceFileFlags.Pinned) & ~SourceFileFlags.Standalone;
			}

			// Do one more forward pass through all the headers, and remove anything that's included more than once. That can happen if we have a referenced symbol as well as 
			// an explicit include, for example.
			HashSet<OutputFile> FinalIncludes = new HashSet<OutputFile>();
			foreach(OutputFileInclude Include in Includes)
			{
				for(int Idx = 0; Idx < Include.FinalFiles.Count; Idx++)
				{
					if(!FinalIncludes.Add(Include.FinalFiles[Idx]))
					{
						Include.FinalFiles.RemoveAt(Idx);
						Idx--;
					}
				}
			}

			// Create the optimized file
			OutputFile OptimizedFile = new OutputFile(InputFile, Includes, Dependencies, ForwardDeclarations);

			// Write the verbose log
			InputFile.LogVerbose("");
			foreach(OutputFile IncludedFile in OptimizedFile.IncludedFiles)
			{
				InputFile.LogVerbose("Output: {0}", IncludedFile.InputFile.Location.FullName);
			}

			// Return the optimized file
			return OptimizedFile; 
		}

		/// <summary>
		/// Creates an output file which represents the same includes as the inpu tfile
		/// </summary>
		/// <param name="InputFile">The input file that this output file corresponds to</param>
		/// <param name="Includes">The active set of includes parsed for this file</param>
		/// <param name="Log">Writer for log messages</param>
		public static OutputFile CreatePassthroughOutputFile(SourceFile InputFile, List<OutputFileInclude> Includes, LineBasedTextWriter Log)
		{
			// Write the state
			InputFile.LogVerbose("InputFile={0}", InputFile.Location.FullName);
			InputFile.LogVerbose("Duplicate.");

			// Reduce the list of includes to those that are required.
			HashSet<SourceFragment> Dependencies = new HashSet<SourceFragment>();
			for(int FragmentIdx = InputFile.Fragments.Length - 1, IncludeIdx = Includes.Count - 1; FragmentIdx >= 0; FragmentIdx--)
			{
				// Update the dependency lists for this fragment
				SourceFragment InputFragment = InputFile.Fragments[FragmentIdx];
				if(InputFragment.Dependencies != null)
				{
					Dependencies.UnionWith(InputFragment.Dependencies);
				}
				Dependencies.Remove(InputFragment);

				// Scan backwards through the list of includes, expanding each include to those which are required
				int MarkupMin = (FragmentIdx == 0)? -1 : InputFragment.MarkupMin;
				for(; IncludeIdx >= 0 && Includes[IncludeIdx].MarkupIdx >= MarkupMin; IncludeIdx--)
				{
					OutputFileInclude Include = Includes[IncludeIdx];
					Include.FinalFiles.Add(Include.TargetFile);
					Dependencies.ExceptWith(Include.TargetFile.IncludedFragments);
					Dependencies.UnionWith(Include.TargetFile.Dependencies);
				}
			}

			// Create the optimized file
			return new OutputFile(InputFile, Includes, new HashSet<SourceFragment>(), new List<Symbol>());
		}

		/// <summary>
		/// Create forward declarations for the given file
		/// </summary>
		/// <param name="InputFile">The file being optimized</param>
		/// <param name="ForwardDeclarations">List of forward declaration lines</param>
		/// <param name="Dependencies">Set of dependencies for the file being optimized</param>
		/// <param name="FwdSymbolToHeader">Map from symbol to header declaring it</param>
		static void AddForwardDeclarations(SourceFile InputFile, List<Symbol> ForwardDeclarations, HashSet<SourceFragment> Dependencies, Dictionary<Symbol, OutputFile> FwdSymbolToHeader)
		{
			// Find all the files which we have true dependencies on. We can assume that all of these will be included.
			HashSet<SourceFile> IncludedFiles = new HashSet<SourceFile>();
			foreach(SourceFragment Fragment in InputFile.Fragments)
			{
				if(Fragment.Dependencies != null)
				{
					IncludedFiles.UnionWith(Fragment.Dependencies.Select(x => x.File));
				}
			}

			// Find all the symbols to forward declare
			HashSet<Symbol> FwdSymbols = new HashSet<Symbol>();
			foreach(SourceFragment Fragment in InputFile.Fragments)
			{
				foreach(Symbol Symbol in Fragment.ReferencedSymbols.Keys)
				{
					if(Symbol.Fragment.File == InputFile || !IncludedFiles.Contains(Symbol.Fragment.File))
					{
						FwdSymbols.Add(Symbol);
					}
				}
			}

			// For each symbol, try to add dependencies on a header which forward declares it, otherwise add the text for it
			foreach(Symbol FwdSymbol in FwdSymbols)
			{
				OutputFile Header;
				if (FwdSymbolToHeader.TryGetValue(FwdSymbol, out Header))
				{
					Dependencies.UnionWith(Header.InputFile.Fragments);
				}
				else
				{
					ForwardDeclarations.Add(FwdSymbol);
				}
			}
		}

		/// <summary>
		/// Build a list of included files which can be expanded out, ignoring those which have already been included
		/// </summary>
		/// <param name="TargetFile">The file to search</param>
		/// <param name="ExpandedReferences">Output list for the uniquely included files from this file</param>
		/// <param name="VisitedFiles">Files which have already been included</param>
		/// <param name="VisitedFragments">Fragments which have already been visited</param>
		static void FindExpandedReferences(OutputFile TargetFile, List<OutputFileReference> ExpandedReferences, HashSet<OutputFile> VisitedFiles, HashSet<SourceFragment> VisitedFragments, bool bIsFirstInclude)
		{
			if(TargetFile != null && (TargetFile.InputFile.Flags & SourceFileFlags.Inline) == 0 && VisitedFiles.Add(TargetFile))
			{
				foreach(OutputFileInclude TargetFileInclude in TargetFile.Includes)
				{
					FindExpandedReferences(TargetFileInclude.TargetFile, ExpandedReferences, VisitedFiles, VisitedFragments, false);
				}
				if((TargetFile.InputFile.Flags & SourceFileFlags.Pinned) == 0 || bIsFirstInclude)
				{
					SourceFragment[] UniqueFragments = TargetFile.IncludedFragments.Except(VisitedFragments).ToArray();
					ExpandedReferences.Add(new OutputFileReference(TargetFile, UniqueFragments));
					VisitedFragments.UnionWith(UniqueFragments);
				}
			}
		}
	}
}
