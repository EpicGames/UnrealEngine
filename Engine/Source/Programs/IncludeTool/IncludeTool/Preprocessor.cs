// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using IncludeTool;
using IncludeTool.Support;
using EpicGames.Core;
using System.Diagnostics;
using System.IO;
using System.Collections;
using System.Collections.Concurrent;

namespace IncludeTool
{
	/// <summary>
	/// Exception class for the preprocessor, which contains the file and position of the code causing an error
	/// </summary>
	class PreprocessorException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Format">Format string, to be passed to String.Format</param>
		/// <param name="Args">Optional argument list for the format string</param>
		public PreprocessorException(string Format, params object[] Args)
			: base(String.Format(Format, Args))
		{
		}
	}

	/// <summary>
	/// Stores information about a defined preprocessor macro
	/// </summary>
	class PreprocessorMacro
	{
		/// <summary>
		/// Name of the macro
		/// </summary>
		public string Name;

		/// <summary>
		/// Parameter names for the macro. The '...' placeholder is represented by the VariadicParameter string.
		/// </summary>
		public List<string> Parameters;

		/// <summary>
		/// Raw list of tokens for this macro
		/// </summary>
		public List<Token> Tokens;

		/// <summary>
		/// The parameter name used for the variadic parameter. This appears in place of '...' in the parameters list. 
		/// </summary>
		public const string VariadicParameter = "__VA_ARGS__";

		/// <summary>
		/// Construct a preprocessor macro
		/// </summary>
		/// <param name="InName">Name of the macro</param>
		/// <param name="InParameters">Parameter list for the macro. Should be null for object macros. Ownership of this list is transferred.</param>
		/// <param name="InTokens">Tokens for the macro. Ownership of this list is transferred.</param>
		public PreprocessorMacro(string InName, List<string> InParameters, List<Token> InTokens)
		{
			Name = InName;
			Parameters = InParameters;
			Tokens = InTokens;
		}

		/// <summary>
		/// Finds the index of a parameter in the parameter list
		/// </summary>
		/// <param name="Parameter">Parameter name to look for</param>
		/// <returns>Index of the parameter, or -1 if it's not found.</returns>
		public int FindParameterIndex(string Parameter)
		{
			for (int Idx = 0; Idx < Parameters.Count; Idx++)
			{
				if (Parameters[Idx] == Parameter)
				{
					return Idx;
				}
			}
			return -1;
		}

		/// <summary>
		/// True if the macro is an object macro
		/// </summary>
		public bool IsObjectMacro
		{
			get { return Parameters == null; }
		}

		/// <summary>
		/// True if the macro is a function macro
		/// </summary>
		public bool IsFunctionMacro
		{
			get { return Parameters != null; }
		}
		
		/// <summary>
		/// The number of required parameters. For variadic macros, the last parameter is optional.
		/// </summary>
		public int MinRequiredParameters
		{
			get { return HasVariableArgumentList? (Parameters.Count - 1) : Parameters.Count; }
		}

		/// <summary>
		/// True if the macro has a variable argument list
		/// </summary>
		public bool HasVariableArgumentList
		{
			get { return Parameters.Count > 0 && Parameters[Parameters.Count - 1] == VariadicParameter; }
		}

		/// <summary>
		/// Converts this macro to a string for debugging
		/// </summary>
		/// <returns>The tokens in this macro</returns>
		public override string ToString()
		{
			StringBuilder Result = new StringBuilder();
			Result.AppendFormat("{0}=", Name);
			if (Tokens.Count > 0)
			{
				Result.Append(Tokens[0].Text);
				for (int Idx = 1; Idx < Tokens.Count; Idx++)
				{
					if(Tokens[Idx].HasLeadingSpace)
					{
						Result.Append(" ");
					}
					Result.Append(Tokens[Idx].Text);
				}
			}
			return Result.ToString();
		}
	}

	/// <summary>
	/// Record for a file context within the preprocessor stack
	/// </summary>
	class PreprocessorFile
	{
		/// <summary>
		/// Filename as specified when included
		/// </summary>
		public string FileName;

		/// <summary>
		/// Reference to the current file
		/// </summary>
		public FileReference Location;

		/// <summary>
		/// The workspace file
		/// </summary>
		public WorkspaceFile WorkspaceFile;

		/// <summary>
		/// Construct a new preprocessor file from the given name
		/// </summary>
		/// <param name="FileName">The filename of the new file</param>
		public PreprocessorFile(string FileName, WorkspaceFile WorkspaceFile)
		{
			this.FileName = FileName;
			this.Location = WorkspaceFile.Location;
			this.WorkspaceFile = WorkspaceFile;
		}

		/// <summary>
		/// Returns the name of this file for visualization in the debugger
		/// </summary>
		/// <returns>Name of this file</returns>
		public override string ToString()
		{
			return FileName;
		}
	}

	/// <summary>
	/// Implementation of a C++ preprocessor. Should be complete and accurate.
	/// </summary>
	class Preprocessor
	{
		/// <summary>
		/// Flags indicating the state of each nested conditional block in a stack.
		/// </summary>
		[Flags]
		enum BranchState
		{
			/// <summary>
			/// The current conditional branch is active.
			/// </summary>
			Active = 0x01,

			/// <summary>
			/// A branch within the current conditional block has been taken. Any #else/#elif blocks will not be taken.
			/// </summary>
			Taken = 0x02,

			/// <summary>
			/// The #else directive for this conditional block has been encountered. An #endif directive is expected next.
			/// </summary>
			Complete = 0x04,

			/// <summary>
			/// This branch is within an active parent branch
			/// </summary>
			ParentIsActive = 0x08,

			/// <summary>
			/// The first condition in this branch was an #if directive (as opposed to an #ifdef or #ifndef directive)
			/// </summary>
			HasIfDirective = 0x10,

			/// <summary>
			/// The first condition in this branch was an #if directive (as opposed to an #ifdef or #ifndef directive)
			/// </summary>
			HasIfdefDirective = 0x20,

			/// <summary>
			/// The first condition in this branch was an #ifndef directive (as opposed to an #ifdef or #if directive)
			/// </summary>
			HasIfndefDirective = 0x40,

			/// <summary>
			/// The branch has an #elif directive
			/// </summary>
			HasElifDirective = 0x80,
		}

		/// <summary>
		/// A directory to search for include paths
		/// </summary>
		[DebuggerDisplay("{Location}")]
		class IncludeDirectory
		{
			/// <summary>
			/// The directory, as specified as a parameter. We keep the original spelling for formatting the preprocessor output to match MSVC.
			/// </summary>
			public DirectoryReference Location;

			/// <summary>
			/// The corresponding workspace directory
			/// </summary>
			public WorkspaceDirectory WorkspaceDirectory;
		}

		/// <summary>
		/// Stack of conditional branch states
		/// </summary>
		Stack<BranchState> Branches = new Stack<BranchState>();

		/// <summary>
		/// Mapping of name to macro definition
		/// </summary>
		Dictionary<string, PreprocessorMacro> NameToMacro = new Dictionary<string, PreprocessorMacro>();

		/// <summary>
		/// The stack of included source files
		/// </summary>
		List<PreprocessorFile> Files = new List<PreprocessorFile>();

		/// <summary>
		/// Include paths to look in
		/// </summary>
		List<IncludeDirectory> IncludeDirectories = new List<IncludeDirectory>();

		/// <summary>
		/// Set of all included files with the #pragma once directive
		/// </summary>
		HashSet<FileReference> PragmaOnceFiles = new HashSet<FileReference>();

		/// <summary>
		/// Value of the __COUNTER__ variable
		/// </summary>
		int Counter;

		/// <summary>
		/// Default constructor
		/// </summary>
		/// <param name="Definitions">Definitions passed in on the command-line</param>
		/// <param name="IncludePaths">Paths to search when resolving include directives</param>
		public Preprocessor(FileReference PreludeFile, IEnumerable<string> Definitions, IEnumerable<DirectoryReference> IncludePaths)
		{
			DateTime Now = DateTime.Now;
			AddSingleTokenMacro("__DATE__", TokenType.StringLiteral, String.Format("\"{0} {1,2} {2}\"", Now.ToString("MMM"), Now.Day, Now.Year));
			AddSingleTokenMacro("__TIME__", TokenType.StringLiteral, "\"" + Now.ToString("HH:mm:ss") + "\"");
			AddSingleTokenMacro("__FILE__", TokenType.StringLiteral, "\"<unknown>\"");
			AddSingleTokenMacro("__LINE__", TokenType.NumericLiteral, "-1");
			AddSingleTokenMacro("__COUNTER__", TokenType.NumericLiteral, "-1");
			AddSingleTokenMacro("CHAR_BIT", TokenType.NumericLiteral, "8"); // Workaround for #include_next not being supported on Linux for limit.h
            AddMacro("__has_cpp_attribute", new List<string> { "_" }, new List<Token> { new Token("1", TokenType.NumericLiteral, TokenFlags.None) }); // Workaround for `__has_cpp_attribute` (a core C++ feature) being defined by a 3rd-party library

			if(PreludeFile != null)
			{
				TextBuffer PreludeText = TextBuffer.FromFile(PreludeFile.FullName);
				PreprocessorMarkup[] PreludeMarkup = PreprocessorMarkup.ParseArray("prelude", PreludeText);
				foreach(PreprocessorMarkup Markup in PreludeMarkup)
				{
					ParseMarkup(Markup.Type, Markup.Tokens, Markup.Location.LineIdx);
				}
			}

			foreach(string Definition in Definitions)
			{
				// Create a token reader for this definition
				TokenReader Reader = new TokenReader($"(predefined value for {Definition})", TextBuffer.FromString(Definition), TextLocation.Origin);

				// Read it into a buffer
				List<Token> Tokens = new List<Token>();
				while (Reader.MoveNext() && Reader.Current.Text != "\n")
				{
					Tokens.Add(Reader.Current);
				}

				// Create the macro for it
				if(Tokens.Count == 0 || Tokens[0].Type != TokenType.Identifier)
				{
					throw new PreprocessorException("Invalid token for define");
				}
				else if(Tokens.Count == 1)
				{
					NameToMacro[Tokens[0].Text] = new PreprocessorMacro(Tokens[0].Text, null, new List<Token> { new Token("1", TokenType.NumericLiteral, TokenFlags.None) });
				}
				else if(Tokens[1].Text == "=")
				{
					NameToMacro[Tokens[0].Text] = new PreprocessorMacro(Tokens[0].Text, null, Tokens.Skip(2).ToList());
				}
				else if(Tokens[1].Text != "(")
				{
					throw new PreprocessorException("Expected '=' token for definition on command-line");
				}
			}

			foreach(DirectoryReference IncludePath in IncludePaths)
			{
				if(Directory.Exists(IncludePath.FullName))
				{
					IncludeDirectories.Add(new IncludeDirectory() { Location = IncludePath, WorkspaceDirectory = Workspace.GetDirectory(IncludePath) });
				}
			}
		}

		/// <summary>
		/// Returns the path to the current file
		/// </summary>
		public PreprocessorFile CurrentFile
		{
			get { return Files[Files.Count - 1]; }
		}

		/// <summary>
		/// Returns an array consisting of the current include stack
		/// </summary>
		/// <returns>Array of the current include stack</returns>
		public FileReference[] CaptureIncludeStack()
		{
			return Files.Select(x => x.Location).ToArray();
		}

		/// <summary>
		/// Determines if the branch that the preprocessor is in is active
		/// </summary>
		/// <returns>True if the branch is active, false otherwise</returns>
		public bool IsBranchActive()
		{
			return Branches.Count == 0 || Branches.Peek().HasFlag(BranchState.Active);
		}

		/// <summary>
		/// Push a file onto the preprocessor stack
		/// </summary>
		/// <param name="File">The file to add</param>
		public bool PushFile(PreprocessorFile File)
		{
			if(!PragmaOnceFiles.Contains(File.Location))
			{
				Files.Add(File);
				if (File.WorkspaceFile.ReadSourceFile().HasHeaderGuard)
				{
					PragmaOnceFiles.Add(File.Location);
				}
				return true;
			}
			return false;
		}

		/// <summary>
		/// Remove the topmost file from the preprocessor stack
		/// </summary>
		public void PopFile()
		{
			Files.RemoveAt(Files.Count - 1);
		}

		/// <summary>
		/// Format a #line directive at the given line
		/// </summary>
		/// <param name="CurrentLine">The current line being parsed</param>
		/// <returns></returns>
		string FormatLineDirective(int CurrentLine)
		{
			return String.Format("#line {0} \"{1}\"", CurrentLine, CurrentFile.FileName.Replace("\\", "\\\\"));
		}

		/// <summary>
		/// Validate and add a macro using the given parameter and token list
		/// </summary>
		/// <param name="Name">Name of the macro</param>
		/// <param name="Parameters">Parameter list for the macro</param>
		/// <param name="Tokens">List of tokens</param>
		void AddMacro(string Name, List<string> Parameters, List<Token> Tokens)
		{
			if(Tokens.Count == 0)
			{
				Tokens.Add(new Token("", TokenType.Placemarker, TokenFlags.None));
			}
			else
			{
				if(Tokens[0].HasLeadingSpace)
				{
					Tokens[0] = new Token(Tokens[0].Text, Tokens[0].Type, Tokens[0].Flags & ~TokenFlags.HasLeadingSpace);
				}
				if (Tokens[0].Text == "##" || Tokens[Tokens.Count - 1].Text == "##")
				{
					throw new PreprocessorException("Invalid use of concatenation at start or end of token sequence");
				}
				if (Parameters == null || Parameters.Count == 0 || Parameters[Parameters.Count - 1] != PreprocessorMacro.VariadicParameter)
				{
					if(Tokens.Any(x => x.Text == PreprocessorMacro.VariadicParameter))
					{
						throw new PreprocessorException("Invalid reference to {0}", PreprocessorMacro.VariadicParameter);
					}
				}
			}
			NameToMacro[Name] = new PreprocessorMacro(Name, Parameters, Tokens);
		}
		
		/// <summary>
		/// Set a predefined macro to a given value
		/// </summary>
		/// <param name="Name">Name of the macro</param>
		/// <param name="Type">Type of the macro token</param>
		/// <param name="Value">Value of the macro</param>
		/// <returns>The created macro</returns>
		void AddSingleTokenMacro(string Name, TokenType Type, string Value)
		{
			Token Token = new Token(Value, Type, TokenFlags.None);
			PreprocessorMacro Macro = new PreprocessorMacro(Name, null, new List<Token> { Token });
			NameToMacro[Name] = Macro;
		}

		/// <summary>
		/// Tries to resolve an include 
		/// </summary>
		/// <param name="Tokens">List of include tokens</param>
		/// <returns>The resolved include file</returns>
		public PreprocessorFile ResolveInclude(List<Token> Tokens, int CurrentLine)
		{
			// Expand macros in the given tokens
			List<Token> ExpandedTokens = new List<Token>();
			ExpandMacros(Tokens, ExpandedTokens, false, CurrentLine);

			// Expand any macros in them and resolve it
			string IncludeToken = Token.Format(ExpandedTokens).Replace("//", "/");
			if (IncludeToken.Length >= 2 && IncludeToken.StartsWith("\"") && IncludeToken.EndsWith("\""))
			{
				return ResolveQuotedInclude(IncludeToken.Substring(1, IncludeToken.Length - 2));
			}
			else if(IncludeToken.StartsWith("<") && IncludeToken.EndsWith(">"))
			{
				return ResolveSystemInclude(IncludeToken.Substring(1, IncludeToken.Length - 2));
			}
			else
			{
				throw new PreprocessorException("Couldn't resolve include '{0}'", IncludeToken);
			}
		}

		/// <summary>
		/// Try to resolve an quoted include against the list of include directories. Uses search order described by https://msdn.microsoft.com/en-us/library/36k2cdd4.aspx.
		/// </summary>
		/// <param name="IncludeText">The path appearing in quotes in an #include directive</param>
		/// <returns>The resolved file</returns>
		public PreprocessorFile ResolveQuotedInclude(string IncludeText)
		{
			// If it's an absolute path, return it immediately
			if(Path.IsPathRooted(IncludeText))
			{
				WorkspaceFile WorkspaceFile = Workspace.GetFile(new FileReference(IncludeText));
				return new PreprocessorFile(IncludeText, WorkspaceFile);
			}

			// Otherwise search through the open file directories
			string[] RelativePathFragments = IncludeText.Split('/', '\\');
			for(int Idx = Files.Count - 1; Idx >= 0; Idx--)
			{
				WorkspaceFile WorkspaceFile;
				if(TryResolveRelativePath(Files[Idx].WorkspaceFile.Directory, RelativePathFragments, out WorkspaceFile))
				{
					string FileName = Path.Combine(Utility.RemoveRelativePaths(Path.GetDirectoryName(Files[Idx].FileName)).ToLowerInvariant(), IncludeText);
					return new PreprocessorFile(FileName, WorkspaceFile);
				}
			}

			// Otherwise fall back to the system search
			return ResolveSystemInclude(IncludeText, RelativePathFragments);
		}

		/// <summary>
		/// Try to resolve a system include against the list of include directories
		/// </summary>
		/// <param name="IncludeText">The path appearing in quotes in an #include directive</param>
		/// <returns>The resolved file</returns>
		public PreprocessorFile ResolveSystemInclude(string IncludeText)
		{
			// If it's an absolute path, return it immediately
			if(Path.IsPathRooted(IncludeText))
			{
				WorkspaceFile WorkspaceFile = Workspace.GetFile(new FileReference(IncludeText));
				return new PreprocessorFile(IncludeText, WorkspaceFile);
			}

			// Otherwise try to resolve a relative path
			return ResolveSystemInclude(IncludeText, IncludeText.Split('/', '\\'));
		}

		/// <summary>
		/// Try to resolve a system include against the list of include directories
		/// </summary>
		/// <param name="IncludeText">The path appearing in quotes in an #include directive</param>
		/// <param name="RelativePathFragments">The components of a relative path, without directory separators</param>
		/// <returns>The resolved file</returns>
		public PreprocessorFile ResolveSystemInclude(string IncludeText, string[] RelativePathFragments)
		{
			foreach(IncludeDirectory IncludeDirectory in IncludeDirectories)
			{
				WorkspaceFile WorkspaceFile;
				if(TryResolveRelativePath(IncludeDirectory.WorkspaceDirectory, RelativePathFragments, out WorkspaceFile))
				{
					string FileName = Path.Combine(IncludeDirectory.Location.FullName, IncludeText);
					return new PreprocessorFile(FileName, WorkspaceFile);
				}
			}
			throw new PreprocessorException("Couldn't resolve include {0}", IncludeText);
		}

		/// <summary>
		/// Tries to get a file at the relative path from a base directory
		/// </summary>
		/// <param name="BaseDirectory">The base directory to search from</param>
		/// <param name="RelativePathFragments">Fragments of the relative path to follow</param>
		/// <param name="Result">The file that was found, if successful</param>
		/// <returns>True if the file was located</returns>
		static bool TryResolveRelativePath(WorkspaceDirectory BaseDirectory, string[] RelativePathFragments, out WorkspaceFile Result)
		{
			WorkspaceDirectory NextDirectory = BaseDirectory;
			for(int Idx = 0; Idx < RelativePathFragments.Length - 1; Idx++)
			{
				if(!NextDirectory.TryGetDirectory(RelativePathFragments[Idx], out NextDirectory))
				{
					Result = null;
					return false;
				}
			}
			return NextDirectory.TryGetFile(RelativePathFragments[RelativePathFragments.Length - 1], out Result);
		}

		/// <summary>
		/// Parse a marked up directive from a file
		/// </summary>
		/// <param name="Type">The markup type</param>
		/// <param name="Tokens">Tokens for the directive</param>
		/// <param name="CurrentLine">The line number being processed</param>
		public void ParseMarkup(PreprocessorMarkupType Type, List<Token> Tokens, int CurrentLine)
		{
			switch(Type)
			{
				case PreprocessorMarkupType.Include:
					throw new PreprocessorException("Include directives should be handled by the caller.");
				case PreprocessorMarkupType.Define:
					ParseDefineDirective(Tokens);
					break;
				case PreprocessorMarkupType.Undef:
					ParseUndefDirective(Tokens);
					break;
				case PreprocessorMarkupType.If:
					ParseIfDirective(Tokens, CurrentLine);
					break;
				case PreprocessorMarkupType.Ifdef:
					ParseIfdefDirective(Tokens);
					break;
				case PreprocessorMarkupType.Ifndef:
					ParseIfndefDirective(Tokens);
					break;
				case PreprocessorMarkupType.Elif:
					ParseElifDirective(Tokens, CurrentLine);
					break;
				case PreprocessorMarkupType.Else:
					ParseElseDirective(Tokens);
					break;
				case PreprocessorMarkupType.Endif:
					ParseEndifDirective(Tokens);
					break;
				case PreprocessorMarkupType.Pragma:
					ParsePragmaDirective(Tokens);
					break;
			}
		}

		/// <summary>
		/// Read a macro definition
		/// </summary>
		/// <param name="Tokens">List of tokens in the directive</param>
		public void ParseDefineDirective(List<Token> Tokens)
		{
			if(IsBranchActive())
			{
				// Check there's a name token
				if(Tokens.Count < 1 || Tokens[0].Type != TokenType.Identifier || Tokens[0].Text == "defined")
				{
					throw new PreprocessorException("Invalid macro name");
				}

				// Read the macro name
				string Name = Tokens[0].Text;
				int TokenIdx = 1;

				// Read the macro parameter list, if there is one
				List<string> Parameters = null;
				if (TokenIdx < Tokens.Count && !Tokens[TokenIdx].HasLeadingSpace && Tokens[TokenIdx].Type == TokenType.LeftParen)
				{
					Parameters = new List<string>();
					if(++TokenIdx == Tokens.Count)
					{
						throw new PreprocessorException("Unexpected end of macro parameter list");
					}
					if(Tokens[TokenIdx].Type != TokenType.RightParen)
					{
						for(;;TokenIdx++)
						{
							// Check there's enough tokens left for a parameter name, plus ',' or ')'
							if(TokenIdx + 2 > Tokens.Count)
							{
								throw new PreprocessorException("Unexpected end of macro parameter list");
							}

							// Check it's a valid name, and add it to the list
							Token NameToken = Tokens[TokenIdx++];
							if(NameToken.Text == "...")
							{
								if(Tokens[TokenIdx].Type != TokenType.RightParen)
								{
									throw new PreprocessorException("Variadic macro arguments must be last in list");
								}
								else
								{
									NameToken = new Token(PreprocessorMacro.VariadicParameter, TokenType.Identifier, NameToken.Flags & TokenFlags.HasLeadingSpace);
								}
							}
							else
							{
								if (NameToken.Type != TokenType.Identifier || NameToken.Text == PreprocessorMacro.VariadicParameter)
								{
									throw new PreprocessorException("Invalid preprocessor token: {0}", NameToken);
								}
								if (Parameters.Contains(NameToken.Text))
								{
									throw new PreprocessorException("'{0}' has already been used as an argument name", NameToken);
								}
							}
							Parameters.Add(NameToken.Text);

							// Read the separator
							Token SeparatorToken = Tokens[TokenIdx];
							if (SeparatorToken.Type == TokenType.RightParen)
							{
								break;
							}
							if (SeparatorToken.Type != TokenType.Comma)
							{
								throw new PreprocessorException("Expected ',' or ')'");
							}
						}
					}
					TokenIdx++;
				}

				// Read the macro tokens
				AddMacro(Name, Parameters, Tokens.Skip(TokenIdx).ToList());
			}
		}

		/// <summary>
		/// Parse an #undef directive
		/// </summary>
		/// <param name="Tokens">List of tokens in the directive</param>
		/// <param name="CurrentLine">The line number being processed</param>
		public void ParseUndefDirective(List<Token> Tokens)
		{
			if(IsBranchActive())
			{
				// Check there's a name token
				if (Tokens.Count != 1)
				{
					throw new PreprocessorException("Expected a single token after #undef");
				}

				// Remove the macro from the list of definitions
				NameToMacro.Remove(Tokens[0].Text);
			}
		}

		/// <summary>
		/// Parse an #if directive
		/// </summary>
		/// <param name="Tokens">List of tokens in the directive</param>
		public void ParseIfDirective(List<Token> Tokens, int CurrentLine)
		{
			BranchState State = BranchState.HasIfDirective;
			if (IsBranchActive())
			{
				// The new branch is within an active branch
				State |= BranchState.ParentIsActive;

				// Read a line into the buffer and expand the macros in it
				List<Token> ExpandedTokens = new List<Token>();
				ExpandMacros(Tokens, ExpandedTokens, true, CurrentLine);

				// Evaluate the condition
				long Result = PreprocessorExpression.Evaluate(ExpandedTokens);
				if (Result != 0)
				{
					State |= BranchState.Active | BranchState.Taken;
				}
			}
			Branches.Push(State);
		}

		/// <summary>
		/// Parse an #ifdef directive
		/// </summary>
		/// <param name="Tokens">List of tokens in the directive</param>
		public void ParseIfdefDirective(List<Token> Tokens)
		{
			BranchState State = BranchState.HasIfdefDirective;
			if (IsBranchActive())
			{
				// The new branch is within an active branch
				State |= BranchState.ParentIsActive;

				// Make sure there's only one token
				if(Tokens.Count != 1 || Tokens[0].Type != TokenType.Identifier)
				{
					throw new PreprocessorException("Missing or invalid macro name for #ifdef directive");
				}

				// Check if the macro is defined
				if(NameToMacro.ContainsKey(Tokens[0].Text))
				{
					State |= BranchState.Active | BranchState.Taken;
				}
			}
			Branches.Push(State);
		}

		/// <summary>
		/// Parse an #ifndef directive
		/// </summary>
		/// <param name="Tokens">List of tokens for this directive</param>
		public void ParseIfndefDirective(List<Token> Tokens)
		{
			BranchState State = BranchState.HasIfndefDirective;
			if (IsBranchActive())
			{
				// The new branch is within an active branch
				State |= BranchState.ParentIsActive;

				// Make sure there's only one token
				if (Tokens.Count != 1 || Tokens[0].Type != TokenType.Identifier)
				{
					throw new PreprocessorException("Missing or invalid macro name for #ifndef directive");
				}

				// Check if the macro is defined
				if(!NameToMacro.ContainsKey(Tokens[0].Text))
				{
					State |= BranchState.Active | BranchState.Taken;
				}
			}
			Branches.Push(State);
		}

		/// <summary>
		/// Parse an #elif directive
		/// </summary>
		/// <param name="Tokens">List of tokens for this directive</param>
		/// <param name="CurrentLine">The line number being processed</param>
		public void ParseElifDirective(List<Token> Tokens, int CurrentLine)
		{
			// Check we're in a branch, and haven't already read an #else directive
			if (Branches.Count == 0)
			{
				throw new PreprocessorException("#elif directive outside conditional block");
			}
			if (Branches.Peek().HasFlag(BranchState.Complete))
			{
				throw new PreprocessorException("#elif directive cannot appear after #else");
			}

			// Pop the current branch state at this depth, so we can test against whether the parent state is enabled
			BranchState State = (Branches.Pop() | BranchState.HasElifDirective) & ~BranchState.Active;
			if (IsBranchActive())
			{
				// Read a line into the buffer and expand the macros in it
				List<Token> ExpandedTokens = new List<Token>();
				ExpandMacros(Tokens, ExpandedTokens, true, CurrentLine);

				// Check we're at the end of a conditional block
				if (!State.HasFlag(BranchState.Taken))
				{
					long Result = PreprocessorExpression.Evaluate(ExpandedTokens);
					if (Result != 0)
					{
						State |= BranchState.Active | BranchState.Taken;
					}
				}
			}
			Branches.Push(State);
		}

		/// <summary>
		/// Parse an #else directive
		/// </summary>
		/// <param name="Tokens">List of tokens in the directive</param>
		public void ParseElseDirective(List<Token> Tokens)
		{
			// Make sure there's nothing else on the line
			if (Tokens.Count > 0)
			{
				throw new PreprocessorException("Garbage after #else directive");
			}

			// Check we're in a branch, and haven't already read an #else directive
			if (Branches.Count == 0)
			{
				throw new PreprocessorException("#else directive without matching #if directive");
			}
			if (Branches.Peek().HasFlag(BranchState.Complete))
			{
				throw new PreprocessorException("Only one #else directive can appear in a conditional block");
			}

			// Check whether to take this branch, but only allow activating if the parent state is active.
			BranchState State = Branches.Pop() & ~BranchState.Active;
			if(IsBranchActive() && !State.HasFlag(BranchState.Taken))
			{
				State |= BranchState.Active | BranchState.Taken;
			}
			Branches.Push(State | BranchState.Complete);
		}

		/// <summary>
		/// Parse an #endif directive
		/// </summary>
		/// <param name="Tokens">List of tokens in the directive</param>
		public void ParseEndifDirective(List<Token> Tokens)
		{
			// Check we're in a branch
			if (Branches.Count == 0)
			{
				throw new PreprocessorException("#endif directive without matching #if/#ifdef/#ifndef directive");
			}

			// Pop the branch off the stack
			Branches.Pop();
		}

		/// <summary>
		/// Parse a #pragma directive
		/// </summary>
		/// <param name="Tokens">List of tokens in the directive</param>
		public void ParsePragmaDirective(List<Token> Tokens)
		{
			if(IsBranchActive())
			{
				if(Tokens.Count == 1 && Tokens[0].Text == "once")
				{
					PragmaOnceFiles.Add(CurrentFile.Location);
				}
			}
		}

		/// <summary>
		/// Expand macros in the given sequence.
		/// </summary>
		/// <param name="InputTokens">Sequence of input tokens</param>
		/// <param name="OutputTokens">List to receive the expanded tokens</param>
		/// <param name="bIsConditional">Whether a conditional expression is being evaluated (and 'defined' expressions are valid)</param>
		public void ExpandMacros(IEnumerable<Token> InputTokens, List<Token> OutputTokens, bool bIsConditional, int CurrentLine)
		{
			List<PreprocessorMacro> IgnoreMacros = new List<PreprocessorMacro>();
			ExpandMacrosRecursively(InputTokens, OutputTokens, bIsConditional, CurrentLine, IgnoreMacros);
		}

		/// <summary>
		/// Expand macros in the given sequence, ignoring previously expanded macro names from a list.
		/// </summary>
		/// <param name="InputTokens">Sequence of input tokens</param>
		/// <param name="OutputTokens">List to receive the expanded tokens</param>
		/// <param name="bIsConditional">Whether a conditional expression is being evaluated (and 'defined' expressions are valid)</param>
		/// <param name="CurrentLine">The current line being expanded which can return the current line at any point</param>
		/// <param name="IgnoreMacros">List of macros to ignore</param>
		void ExpandMacrosRecursively(IEnumerable<Token> InputTokens, List<Token> OutputTokens, bool bIsConditional, int CurrentLine, List<PreprocessorMacro> IgnoreMacros)
		{
			IEnumerator<Token> InputEnumerator = InputTokens.GetEnumerator();
			if(InputEnumerator.MoveNext())
			{
				for(;;)
				{
					if(!ReadExpandedToken(InputEnumerator, OutputTokens, bIsConditional, () => CurrentLine, IgnoreMacros))
					{
						break;
					}
				}
			}
		}

		/// <summary>
		/// Merges an optional leading space flag into the given token (recycling the original token if possible).
		/// </summary>
		/// <param name="Token">The token to merge a leading space into</param>
		/// <param name="bHasLeadingSpace">The leading space flag</param>
		/// <returns>New token with the leading space flag set, or the existing token</returns>
		Token MergeLeadingSpace(Token Token, bool bHasLeadingSpace)
		{
			Token Result = Token;
			if(bHasLeadingSpace && !Result.HasLeadingSpace)
			{
				Result = new Token(Result.Text, Result.Type, Result.Flags | TokenFlags.HasLeadingSpace);
			}
			return Result;
		}

		/// <summary>
		/// Read a token from an enumerator and substitute it if it's a macro or 'defined' expression (reading more tokens as necessary to complete the expression).
		/// </summary>
		/// <param name="InputEnumerator">The enumerator of input tokens</param>
		/// <param name="OutputTokens">List to receive the expanded tokens</param>
		/// <param name="bIsConditional">Whether a conditional expression is being evaluated (and 'defined' expressions are valid)</param>
		/// <param name="CurrentLine">Callback which can return the current line at any point</param>
		/// <param name="IgnoreMacros">List of macros to ignore</param>
		/// <returns>Result from calling the enumerator's MoveNext() method</returns>
		bool ReadExpandedToken(IEnumerator<Token> InputEnumerator, List<Token> OutputTokens, bool bIsConditional, Func<int> CurrentLine, List<PreprocessorMacro> IgnoreMacros)
		{
			// Capture the first token, then move to the next
			OutputTokens.Add(InputEnumerator.Current);
			bool bMoveNext = InputEnumerator.MoveNext();

			// If it's an identifier, try to resolve it as a macro
			if (OutputTokens[OutputTokens.Count - 1].Text == "defined" && bIsConditional)
			{
				// Remove the 'defined' keyword
				OutputTokens.RemoveAt(OutputTokens.Count - 1);

				// Make sure there's another token
				if (!bMoveNext)
				{
					throw new PreprocessorException("Invalid syntax for 'defined' expression");
				}

				// Check for the form 'defined identifier'
				Token NameToken;
				if(InputEnumerator.Current.Type == TokenType.Identifier)
				{
					NameToken = InputEnumerator.Current;
				}
				else
				{
					// Otherwise assume the form 'defined ( identifier )'
					if(InputEnumerator.Current.Type != TokenType.LeftParen || !InputEnumerator.MoveNext() || InputEnumerator.Current.Type != TokenType.Identifier)
					{
						throw new PreprocessorException("Invalid syntax for 'defined' expression");
					}
					NameToken = InputEnumerator.Current;
					if(!InputEnumerator.MoveNext() || InputEnumerator.Current.Type != TokenType.RightParen)
					{
						throw new PreprocessorException("Invalid syntax for 'defined' expression");
					}
				}

				// Insert a token for whether it's defined or not
				OutputTokens.Add(new Token(NameToMacro.ContainsKey(NameToken.Text)? "1" : "0", TokenType.NumericLiteral, TokenFlags.None));
				bMoveNext = InputEnumerator.MoveNext();
			}
			else
			{
				// Repeatedly try to expand the last token into the list
				while(OutputTokens[OutputTokens.Count - 1].Type == TokenType.Identifier && !OutputTokens[OutputTokens.Count - 1].Flags.HasFlag(TokenFlags.DisableExpansion))
				{
					// Try to get a macro for the current token
					PreprocessorMacro Macro;
					if (!NameToMacro.TryGetValue(OutputTokens[OutputTokens.Count - 1].Text, out Macro) || IgnoreMacros.Contains(Macro))
					{
						break;
					}
					if(Macro.IsFunctionMacro && (!bMoveNext || InputEnumerator.Current.Type != TokenType.LeftParen))
					{
						break;
					}

					// Remove the macro name from the output list
					bool bHasLeadingSpace = OutputTokens[OutputTokens.Count - 1].HasLeadingSpace;
					OutputTokens.RemoveAt(OutputTokens.Count - 1);

					// Save the initial number of tokens in the list, so we can tell if it expanded
					int NumTokens = OutputTokens.Count;

					// If it's an object macro, expand it immediately into the output buffer
					if(Macro.IsObjectMacro)
					{
						// Expand the macro tokens into the output buffer
						ExpandObjectMacro(Macro, OutputTokens, bIsConditional, CurrentLine(), IgnoreMacros);
					}
					else
					{
						// Read balanced token for argument list
						List<Token> ArgumentTokens = new List<Token>();
						bMoveNext = ReadBalancedToken(InputEnumerator, ArgumentTokens);

						// Expand the macro tokens into the output buffer
						ExpandFunctionMacro(Macro, ArgumentTokens, OutputTokens, bIsConditional, CurrentLine(), IgnoreMacros);
					}

					// If the macro expanded to nothing, quit
					if(OutputTokens.Count <= NumTokens)
					{
						break;
					}

					// Make sure the space is propagated to the expanded macro
					OutputTokens[NumTokens] = MergeLeadingSpace(OutputTokens[NumTokens], bHasLeadingSpace);

					// Mark any tokens matching the macro name as not to be expanded again. This can happen with recursive object macros, eg. #define DWORD ::DWORD
					for(int Idx = NumTokens; Idx < OutputTokens.Count; Idx++)
					{
						if(OutputTokens[Idx].Type == TokenType.Identifier && OutputTokens[Idx].Text == Macro.Name)
						{
							OutputTokens[Idx] = new Token(OutputTokens[Idx].Text, TokenType.Identifier, OutputTokens[Idx].Flags | TokenFlags.DisableExpansion);
						}
					}
				}
			}
            return bMoveNext;
		}

		/// <summary>
		/// Expand an object macro
		/// </summary>
		/// <param name="Macro">The functional macro</param>
		/// <param name="OutputTokens">The list to receive the output tokens</param>
		/// <param name="bIsConditional">Whether the macro is being expanded in a conditional context, allowing use of the 'defined' keyword</param>
		/// <param name="CurrentLine">The line number being processed</param>
		/// <param name="IgnoreMacros">List of macros currently being expanded, which should be ignored for recursion</param>
		void ExpandObjectMacro(PreprocessorMacro Macro, List<Token> OutputTokens, bool bIsConditional, int CurrentLine, List<PreprocessorMacro> IgnoreMacros)
		{
			// Special handling for the __LINE__ directive, since we need an updated line number for the current token
			if(Macro.Name == "__FILE__")
			{
				Token Token = new Token(String.Format("\"{0}\"", CurrentFile.FileName.Replace("\\", "\\\\")), TokenType.StringLiteral, TokenFlags.None);
				OutputTokens.Add(Token);
			}
			else if(Macro.Name == "__LINE__")
			{
				Token Token = new Token((CurrentLine + 1).ToString(), TokenType.NumericLiteral, TokenFlags.None);
				OutputTokens.Add(Token);
			}
			else if(Macro.Name == "__COUNTER__")
			{
				Token Token = new Token((Counter++).ToString(), TokenType.NumericLiteral, TokenFlags.None);
				OutputTokens.Add(Token);
			}
			else
			{
				int OutputTokenCount = OutputTokens.Count;

				// Expand all the macros
				IgnoreMacros.Add(Macro);
				ExpandMacrosRecursively(Macro.Tokens, OutputTokens, bIsConditional, CurrentLine, IgnoreMacros);
				IgnoreMacros.RemoveAt(IgnoreMacros.Count - 1);

				// Concatenate any adjacent tokens
				for(int Idx = OutputTokenCount + 1; Idx < OutputTokens.Count - 1; Idx++)
				{
					if(OutputTokens[Idx].Text == "##")
					{
						OutputTokens[Idx - 1] = Token.Concatenate(OutputTokens[Idx - 1], OutputTokens[Idx + 1]);
						OutputTokens.RemoveRange(Idx, 2);
						Idx--;
					}
				}
			}
		}

		/// <summary>
		/// Expand a function macro
		/// </summary>
		/// <param name="Macro">The functional macro</param>
		/// <param name="OutputTokens">The list to receive the output tokens</param>
		/// <param name="bIsConditional">Whether the macro is being expanded in a conditional context, allowing use of the 'defined' keyword</param>
		/// <param name="CurrentLine">The line number being processed</param>
		/// <param name="IgnoreMacros">List of macros currently being expanded, which should be ignored for recursion</param>
		void ExpandFunctionMacro(PreprocessorMacro Macro, List<Token> ArgumentListTokens, List<Token> OutputTokens, bool bIsConditional, int CurrentLine, List<PreprocessorMacro> IgnoreMacros)
		{
			// Replace any newlines with spaces, and merge them with the following token
			for(int Idx = 0; Idx < ArgumentListTokens.Count; Idx++)
			{
				if(ArgumentListTokens[Idx].Text == "\n")
				{
					if(Idx + 1 < ArgumentListTokens.Count)
					{
						ArgumentListTokens[Idx + 1] = MergeLeadingSpace(ArgumentListTokens[Idx + 1], true);
					}
					ArgumentListTokens.RemoveAt(Idx--);
				}
			}

			// Split the arguments out into separate lists
			List<List<Token>> Arguments = new List<List<Token>>();
			if(ArgumentListTokens.Count > 2)
			{
				for(int Idx = 1;;Idx++)
				{
					if (!Macro.HasVariableArgumentList || Arguments.Count < Macro.Parameters.Count)
					{
						Arguments.Add(new List<Token>());
					}

					List<Token> Argument = Arguments[Arguments.Count - 1];

					int InitialIdx = Idx;
					while (Idx < ArgumentListTokens.Count - 1 && ArgumentListTokens[Idx].Text != ",")
					{
						if (!ReadBalancedToken(ArgumentListTokens, ref Idx, Argument))
						{
							throw new PreprocessorException("Invalid argument");
						}
					}

					if(Argument.Count > 0 && Arguments[Arguments.Count - 1][0].HasLeadingSpace)
					{
						Argument[0] = new Token(Argument[0].Text, Argument[0].Type, Argument[0].Flags & ~TokenFlags.HasLeadingSpace);
					}

					bool bHasLeadingSpace = false;
					for(int TokenIdx = 0; TokenIdx < Argument.Count; TokenIdx++)
					{
						if(Argument[TokenIdx].Text.Length == 0)
						{
							bHasLeadingSpace |= Argument[TokenIdx].HasLeadingSpace;
							Argument.RemoveAt(TokenIdx--);
						}
						else
						{
							Argument[TokenIdx] = MergeLeadingSpace(Argument[TokenIdx], bHasLeadingSpace);
							bHasLeadingSpace = false;
						}
					}

					if (Argument.Count == 0)
					{
						Argument.Add(new Token("", TokenType.Placemarker, TokenFlags.None));
						Argument.Add(new Token("", TokenType.Placemarker, bHasLeadingSpace? TokenFlags.HasLeadingSpace : TokenFlags.None));
					}

					if(Idx == ArgumentListTokens.Count - 1)
					{
						break;
					}

					if (ArgumentListTokens[Idx].Text != ",")
					{
						throw new PreprocessorException("Expected ',' between arguments");
					}

					if (Macro.HasVariableArgumentList && Arguments.Count == Macro.Parameters.Count && Idx < ArgumentListTokens.Count - 1)
					{
						Arguments[Arguments.Count - 1].Add(ArgumentListTokens[Idx]);
					}
				}
			}

			// Add an empty variable argument if one was not specified
			if (Macro.HasVariableArgumentList && Arguments.Count == Macro.Parameters.Count - 1)
			{
				Arguments.Add(new List<Token> { new Token("", TokenType.Placemarker, TokenFlags.None) });
			}

			// Validate the argument list
			if (Arguments.Count != Macro.Parameters.Count)
			{
				throw new PreprocessorException("Incorrect number of arguments to macro");
			}

			// Expand each one of the arguments
			List<List<Token>> ExpandedArguments = new List<List<Token>>();
			for (int Idx = 0; Idx < Arguments.Count; Idx++)
			{
				List<Token> NewArguments = new List<Token>();
				ExpandMacrosRecursively(Arguments[Idx], NewArguments, bIsConditional, CurrentLine, IgnoreMacros);
				ExpandedArguments.Add(NewArguments);
			}

			// Substitute all the argument tokens
			List<Token> ExpandedTokens = new List<Token>();
			for(int Idx = 0; Idx < Macro.Tokens.Count; Idx++)
			{
				Token Token = Macro.Tokens[Idx];
				if(Token.Text == "#" && Idx + 1 < Macro.Tokens.Count)
				{
					// Stringizing operator
					int ParamIdx = Macro.FindParameterIndex(Macro.Tokens[++Idx].Text);
					if (ParamIdx == -1)
					{
						throw new PreprocessorException("{0} is not an argument name", Macro.Tokens[Idx].Text);
					}
					ExpandedTokens.Add(new Token(String.Format("\"{0}\"", Token.Format(Arguments[ParamIdx]).Replace("\\", "\\\\").Replace("\"", "\\\"")), TokenType.StringLiteral, Token.Flags & TokenFlags.HasLeadingSpace));
				}
				else if(Macro.HasVariableArgumentList && Idx + 2 < Macro.Tokens.Count && Token.Text == "," && Macro.Tokens[Idx + 1].Text == "##" && Macro.Tokens[Idx + 2].Text == PreprocessorMacro.VariadicParameter)
				{
					// Special MSVC/GCC extension: ', ## __VA_ARGS__' removes the comma if __VA_ARGS__ is empty. MSVC seems to format the result with a forced space.
					List<Token> ExpandedArgument = ExpandedArguments[ExpandedArguments.Count - 1];
					if(ExpandedArgument.Any(x => x.Text.Length > 0))
					{
						ExpandedTokens.Add(Token);
						AppendTokensWithWhitespace(ExpandedTokens, ExpandedArgument, false);
						Idx += 2;
					}
					else
					{
						ExpandedTokens.Add(new Token("", TokenType.Placemarker, Token.Flags & TokenFlags.HasLeadingSpace));
						ExpandedTokens.Add(new Token("", TokenType.Placemarker, TokenFlags.HasLeadingSpace));
						Idx += 2;
					}
				}
				else if (Token.Type == TokenType.Identifier)
				{
					// Expand a parameter
					int ParamIdx = Macro.FindParameterIndex(Token.Text);
					if(ParamIdx == -1)
					{
						ExpandedTokens.Add(Token);
					}
					else if(Idx > 0 && Macro.Tokens[Idx - 1].Text == "##")
					{
						AppendTokensWithWhitespace(ExpandedTokens, Arguments[ParamIdx], Token.HasLeadingSpace);
					}
					else
					{
						AppendTokensWithWhitespace(ExpandedTokens, ExpandedArguments[ParamIdx], Token.HasLeadingSpace);
					}
				}
				else
				{
					ExpandedTokens.Add(Token);
				}
			}

			// Concatenate adjacent tokens
			for (int Idx = 1; Idx < ExpandedTokens.Count - 1; Idx++)
			{
				if(ExpandedTokens[Idx].Text == "##")
				{
					Token ConcatenatedToken = Token.Concatenate(ExpandedTokens[Idx - 1], ExpandedTokens[Idx + 1]);
					ExpandedTokens.RemoveRange(Idx, 2);
					ExpandedTokens[--Idx] = ConcatenatedToken;
				}
			}

			// Finally, return the expansion of this
			IgnoreMacros.Add(Macro);
			ExpandMacrosRecursively(ExpandedTokens, OutputTokens, bIsConditional, CurrentLine, IgnoreMacros);
			IgnoreMacros.RemoveAt(IgnoreMacros.Count - 1);
		}

		/// <summary>
		/// Appends a list of tokens to another list, setting the leading whitespace flag to the given value
		/// </summary>
		/// <param name="OutputTokens">List to receive the appended tokens</param>
		/// <param name="InputTokens">List of tokens to append</param>
		/// <param name="bHasLeadingSpace">Whether there is space before the first token</param>
		void AppendTokensWithWhitespace(List<Token> OutputTokens, List<Token> InputTokens, bool bHasLeadingSpace)
		{
			if(InputTokens.Count > 0)
			{
				OutputTokens.Add(MergeLeadingSpace(InputTokens[0], bHasLeadingSpace));
				OutputTokens.AddRange(InputTokens.Skip(1));
			}
		}

		/// <summary>
		/// Copies a single token from one list of tokens to another, or if it's an opening parenthesis, the entire subexpression until the closing parenthesis.
		/// </summary>
		/// <param name="InputTokens">The input token list</param>
		/// <param name="InputIdx">First token index in the input token list. Set to the last uncopied token index on return.</param>
		/// <param name="OutputTokens">List to recieve the output tokens</param>
		/// <returns>True if a balanced expression was read, or false if the end of the list was encountered before finding a matching token</returns>
		bool ReadBalancedToken(List<Token> InputTokens, ref int InputIdx, List<Token> OutputTokens)
		{
			// Copy a single token to the output list
			Token Token = InputTokens[InputIdx++];
			OutputTokens.Add(Token);

			// If it was the start of a subexpression, copy until the closing parenthesis
			if (Token.Type == TokenType.LeftParen)
			{
				// Copy the contents of the subexpression
				for (;;)
				{
					if (InputIdx == InputTokens.Count)
					{
						return false;
					}
					if (InputTokens[InputIdx].Type == TokenType.RightParen)
					{
						break;
					}
					if (!ReadBalancedToken(InputTokens, ref InputIdx, OutputTokens))
					{
						return false;
					}
				}

				// Copy the closing parenthesis
				Token = InputTokens[InputIdx++];
				OutputTokens.Add(Token);
			}
			return true;
		}

		/// <summary>
		/// Copies a single token from one list of tokens to another, or if it's an opening parenthesis, the entire subexpression until the closing parenthesis.
		/// </summary>
		/// <param name="InputTokens">The input token list</param>
		/// <param name="InputIdx">First token index in the input token list. Set to the last uncopied token index on return.</param>
		/// <param name="OutputTokens">List to recieve the output tokens</param>
		/// <returns>True if a balanced expression was read, or false if the end of the list was encountered before finding a matching token</returns>
		bool ReadBalancedToken(IEnumerator<Token> InputEnumerator, List<Token> OutputTokens)
		{
			// Copy a single token to the output list
			Token Token = InputEnumerator.Current;
			bool bMoveNext = InputEnumerator.MoveNext();
			OutputTokens.Add(Token);

			// If it was the start of a subexpression, copy until the closing parenthesis
			if (Token.Type == TokenType.LeftParen)
			{
				// Copy the contents of the subexpression
				for (;;)
				{
					if (!bMoveNext)
					{
						throw new PreprocessorException("Unbalanced token sequence");
					}
					if (InputEnumerator.Current.Type == TokenType.RightParen)
					{
						OutputTokens.Add(InputEnumerator.Current);
						bMoveNext = InputEnumerator.MoveNext();
						break;
					}
					bMoveNext = ReadBalancedToken(InputEnumerator, OutputTokens);
				}
			}
			return bMoveNext;
		}

		/// <summary>
		/// Create a file which represents the compiler definitions
		/// </summary>
		/// <param name="PreludeFile"></param>
		/// <param name="CompileEnvironment"></param>
		/// <param name="WorkingDir"></param>
		/// <param name="Definitions"></param>
		public static void CreatePreludeFile(FileReference PreludeFile, CompileEnvironment CompileEnvironment, DirectoryReference WorkingDir)
		{
			// Spawn the compiler and write the prelude file
			string CompilerName = CompileEnvironment.Compiler.GetFileName().ToLowerInvariant();
			if(CompileEnvironment.CompilerType == CompilerType.Clang)
			{
				// Create a dummy input file
				FileReference InputFile = PreludeFile.ChangeExtension(".in.cpp");
				File.WriteAllText(InputFile.FullName, "");

				// Spawn clang and parse the output
				using (StreamWriter Writer = new StreamWriter(PreludeFile.FullName))
				{
					Utility.Run(CompileEnvironment.Compiler, String.Join(" ", CompileEnvironment.Options.Where(x => x.Name == "-target").Select(x => x.ToString())) + " -dM -E -x c++ " + InputFile.FullName, WorkingDir, new LineBasedTextWriterWrapper(Writer));
				}
			}
			else if(CompileEnvironment.CompilerType == CompilerType.VisualC)
			{
				// Create a query file
				FileReference InputFile = PreludeFile.ChangeExtension(".in");
				using (StreamWriter InputWriter = new StreamWriter(InputFile.FullName))
				{
					InputWriter.WriteLine("#define STRINGIZE_2(x) #x");
					InputWriter.WriteLine("#define STRINGIZE(x) STRINGIZE_2(x)");

					string[] MacroNames = Properties.Resources.Prelude.Split('\n').Select(x => x.Trim()).Where(x => x.Length > 0 && !x.StartsWith(";")).ToArray();
					foreach (string MacroName in MacroNames)
					{
						InputWriter.WriteLine("");
						InputWriter.WriteLine("#ifdef {0}", MacroName);
						InputWriter.WriteLine("#pragma message(\"#define {0} \" STRINGIZE({0}))", MacroName);
						InputWriter.WriteLine("#endif");
					}
				}

				// Invoke the compiler and capture the output into a string
				StringWriter OutputWriter = new StringWriter();
				string FullCommandLine = String.Format("{0} /Zs /TP {1}", CompileEnvironment.GetCommandLine(), InputFile.FullName);
				Utility.Run(CompileEnvironment.Compiler, FullCommandLine, WorkingDir, new LineBasedTextWriterWrapper(OutputWriter));

				// Filter the output
				using (StreamWriter Writer = new StreamWriter(PreludeFile.FullName))
				{
					foreach(string Line in OutputWriter.ToString().Split('\n').Select(x => x.Trim()).Where(x => x.Length > 0))
					{
						if(Line.StartsWith("#define"))
						{
							Writer.WriteLine(Line);
						}
						else if(Line.Trim() != InputFile.GetFileName() && !Line.StartsWith("time("))
						{
							throw new Exception(String.Format("Unexpected output from compiler: {0}", OutputWriter.ToString()));
						}
					}
				}
				}
				else
				{
					throw new NotImplementedException("Unknown compiler: " + CompilerName);
				}
			}
		
		/// <summary>
		/// Preprocess the current file and write it to the given output
		/// </summary>
		/// <param name="InputFile">The file to read from</param>
		/// <param name="OutputFile">File to output to</param>
		public void PreprocessFile(string InputFileName, FileReference OutputFile, IEnumerable<FileReference> ForceIncludeFiles)
		{
			using (StreamWriter Writer = new StreamWriter(OutputFile.FullName))
			{
				List<FileReference> InputFiles = new List<FileReference>();
				InputFiles.AddRange(ForceIncludeFiles);
				InputFiles.Add(new FileReference(InputFileName));

				foreach(FileReference InputFile in InputFiles)
				{
					WorkspaceFile InputWorkspaceFile = Workspace.GetFile(InputFile);
					PreprocessFile(new PreprocessorFile(InputFile.FullName, InputWorkspaceFile), Writer);
				}
			}
		}

		/// <summary>
		/// Preprocess the current file to the given output stream
		/// </summary>
		/// <param name="InputFile">The file to read from</param>
		/// <param name="Writer">Stream to output to</param>
		bool PreprocessFile(PreprocessorFile InputFile, TextWriter Writer)
		{
			bool bResult = false;
			if(PushFile(InputFile))
			{
				FileReference InputFileLocation = InputFile.Location;
				SourceFile SourceFile = InputFile.WorkspaceFile.ReadSourceFile();

				// Always write a #line directive at the start of the file
				Writer.WriteLine(FormatLineDirective(1));

				// Write the rest of the file
				TextLocation Location = TextLocation.Origin;
				foreach (PreprocessorMarkup Markup in SourceFile.Markup)
				{
					// Write everything since the last item
					if (Markup.Location > Location)
					{
						if (IsBranchActive())
						{
							WriteText(SourceFile.Text, Location, Markup.Location, Writer);
						}
						else
						{
							WriteBlankedText(SourceFile.Text, Location, Markup.Location, Writer);
						}
						Location = Markup.Location;
					}

					// Check the type of this markup
					if (Markup.Type == PreprocessorMarkupType.Text)
					{
						// Write the expanded macro tokens if the current block is active, or blank lines if not
						if (IsBranchActive())
						{
							WriteExpandedText(SourceFile.Location.FullName, SourceFile.Text, Location, Markup.EndLocation, Writer);
						}
						else
						{
							WriteBlankedText(SourceFile.Text, Location, Markup.EndLocation, Writer);
						}
						Location = Markup.EndLocation;
					}
					else
					{
						// Capture initial state that we need later on
						bool bActive = IsBranchActive();

						// Check whether we want a #line directive after this point
						bool bOutputLineDirective = false;
						if(Markup.Type == PreprocessorMarkupType.Endif)
						{
							if(Branches.Peek().HasFlag(BranchState.HasIfDirective) && Branches.Peek().HasFlag(BranchState.ParentIsActive))
							{
								bOutputLineDirective = true;
							}
							if(Branches.Peek().HasFlag(BranchState.HasIfdefDirective) && Branches.Peek().HasFlag(BranchState.ParentIsActive) && Branches.Peek().HasFlag(BranchState.Taken))
							{
								bOutputLineDirective = true;
							}
							if(Branches.Peek().HasFlag(BranchState.HasIfndefDirective) && Branches.Peek().HasFlag(BranchState.ParentIsActive) && Branches.Peek().HasFlag(BranchState.Taken))
							{
								bOutputLineDirective = true;
							}
						}

						// Parse the markup
						if (Markup.Type == PreprocessorMarkupType.Include)
						{
							bOutputLineDirective |= WriteInclude(Markup.Tokens, Markup.Location.LineIdx, Writer);
						}
						else
						{
							ParseMarkup(Markup.Type, Markup.Tokens, Markup.Location.LineIdx);
						}

						// Find the newline token (note - may be skipping over escaped newlines here)
						Location = Markup.EndLocation;

						// Write the contents of this markup, if necessary
						if (IsBranchActive())
						{
							if(Markup.Type == PreprocessorMarkupType.Else)
							{
								if(Branches.Peek().HasFlag(BranchState.HasIfDirective) || Branches.Peek().HasFlag(BranchState.HasElifDirective))
								{
									bOutputLineDirective = true;
								}
							}
							else if (Markup.Type == PreprocessorMarkupType.Elif)
							{
								if(!Branches.Peek().HasFlag(BranchState.HasIfdefDirective))
								{
									bOutputLineDirective = true;
								}
							}
						}
						else
						{
							if(Markup.Type == PreprocessorMarkupType.Elif)
							{
								if(!Branches.Peek().HasFlag(BranchState.Taken) && Branches.Peek().HasFlag(BranchState.ParentIsActive) && !Branches.Peek().HasFlag(BranchState.HasIfdefDirective))
								{
									bOutputLineDirective = true;
								}
							}
							if(Markup.Type == PreprocessorMarkupType.Else)
							{
								if(!Branches.Peek().HasFlag(BranchState.Taken) && Branches.Peek().HasFlag(BranchState.ParentIsActive))// && !Branches.Peek().HasFlag(BranchState.InitialIfCondition))
								{
									bOutputLineDirective = true;
								}
							}
						}

						// Write any text or escaped newlines for this markup
						if (bActive && Markup.Type == PreprocessorMarkupType.Pragma && (Markup.Tokens[0].Text != "pop_macro" && Markup.Tokens[0].Text != "push_macro"))
						{
							List<Token> ExpandedTokens = new List<Token>();
							ExpandMacros(Markup.Tokens, ExpandedTokens, false, Markup.Location.LineIdx);
							Writer.Write("#pragma {0}", Token.Format(ExpandedTokens));
						}
						else if(!bActive || Markup.Type == PreprocessorMarkupType.Define)
						{
							for (int EscapedLineIdx = Markup.Location.LineIdx; EscapedLineIdx < Markup.EndLocation.LineIdx - 1; EscapedLineIdx++)
							{
								Writer.WriteLine();
							}
						}

						// Output the line directive
						if(bOutputLineDirective)
						{
							Writer.WriteLine(FormatLineDirective(Markup.EndLocation.LineIdx + 1));
						}
						else
						{
							Writer.WriteLine();
						}
					}
				}
				WriteText(SourceFile.Text, Location, SourceFile.Text.End, Writer);

				// Automatically treat header guards as #pragma once style directives
				if (SourceFile.HasHeaderGuard)
				{
					PragmaOnceFiles.Add(SourceFile.Location);
				}

				// Pop the file from the include stack
				PopFile();
				bResult = true;
			}
			return bResult;
		}

		/// <summary>
		/// Resolve an included file and write it to an output stream
		/// </summary>
		/// <param name="Tokens">Tokens for the include directive</param>
		/// <param name="CurrentLine">The current line</param>
		/// <param name="Writer">Writer for the output text</param>
		/// <returns>True if an include was written, false if the current branch is not active, or the file has already been included</returns>
		bool WriteInclude(List<Token> Tokens, int CurrentLine, TextWriter Writer)
		{
			bool bResult = false;
			if (IsBranchActive())
			{
				PreprocessorFile IncludeFile = ResolveInclude(Tokens, CurrentLine);
				bResult = PreprocessFile(IncludeFile, Writer);
			}
			return bResult;
		}

		/// <summary>
		/// Preprocess the current file to the given output stream
		/// </summary>
		/// <param name="Text">The text buffer for the current file</param>
		/// <param name="Location">Start of the region to copy to the output stream</param>
		/// <param name="EndLocation">End of the region to copy to the output stream</param>
		/// <param name="Writer">Writer for the output text</param>
		void WriteText(TextBuffer Text, TextLocation Location, TextLocation EndLocation, TextWriter Writer)
		{
			// Write the whitespace
			while(EndLocation.LineIdx > Location.LineIdx)
			{
				if(Text.Lines[Location.LineIdx].EndsWith("*\\"))
				{
					// Hacky fix to make preprocessed output the same as MSVC. When writing a multiline comment ending with *\ (as appears in copyright boilerplate for
					// headers like winver.h), both characters are removed and the * is pushed into the start of the next line.
					Writer.WriteLine(Text.Lines[Location.LineIdx].Substring(Location.ColumnIdx, Text.Lines[Location.LineIdx].Length - 2));
					Writer.Write("*");
				}
				else
				{
					Writer.WriteLine(Text.Lines[Location.LineIdx].Substring(Location.ColumnIdx));
				}
				Location.ColumnIdx = 0;
				Location.LineIdx++;
			}

			// Write the last line
			if(Location.ColumnIdx < EndLocation.ColumnIdx)
			{
				Writer.Write(Text.Lines[Location.LineIdx].Substring(Location.ColumnIdx, EndLocation.ColumnIdx - Location.ColumnIdx));
			}
		}

		/// <summary>
		/// Preprocess the current file to the given output stream
		/// </summary>
		/// <param name="Text">The text buffer for the current file</param>
		/// <param name="Location">Start of the region to copy to the output stream</param>
		/// <param name="EndLocation">End of the region to copy to the output stream</param>
		/// <param name="Writer">Writer for the output text</param>
		void WriteBlankedText(TextBuffer Text, TextLocation Location, TextLocation EndLocation, TextWriter Writer)
		{
			// Write the whitespace
			while(EndLocation.LineIdx > Location.LineIdx)
			{
				Writer.WriteLine();
				Location.ColumnIdx = 0;
				Location.LineIdx++;
			}

			// Write the last line
			if(Location.ColumnIdx < EndLocation.ColumnIdx)
			{
				//Writer.Write(Text.Lines[Location.LineIdx].Substring(Location.ColumnIdx, EndLocation.ColumnIdx - Location.ColumnIdx));
			}
		}

		/// <summary>
		/// Write a portion of the given file to the output stream, expanding macros in it
		/// </summary>
		/// <param name="FileName">The file being parsed</param>
		/// <param name="Text">The text buffer for the current file</param>
		/// <param name="Location">Start of the region to copy to the output stream</param>
		/// <param name="EndLocation">End of the region to copy to the output stream</param>
		/// <param name="Writer">Writer for the output text</param>
		void WriteExpandedText(string FileName, TextBuffer Text, TextLocation Location, TextLocation EndLocation, TextWriter Writer)
		{
			int ExpectedLineIdx = Location.LineIdx;

			// Expand any macros in this block of tokens
			TokenReader Reader = new TokenReader(FileName, Text, Location);
			for(bool bMoveNext = Reader.MoveNext(); bMoveNext; )
			{
				// Write the whitespace to the output stream
				WriteText(Text, Reader.TokenWhitespaceLocation, Reader.TokenLocation, Writer);
				ExpectedLineIdx += Reader.TokenLocation.LineIdx - Reader.TokenWhitespaceLocation.LineIdx;

				// Check if we've reached the end
				if(Reader.TokenLocation >= EndLocation)
				{
					break;
				}

				// Write out the current token
				if(Reader.Current.Text == "\n")
				{
					Writer.WriteLine();
					if(++ExpectedLineIdx != Reader.TokenEndLocation.LineIdx)
					{
						ExpectedLineIdx = Reader.TokenEndLocation.LineIdx;
						Writer.WriteLine(FormatLineDirective(ExpectedLineIdx + 1));
					}
					bMoveNext = Reader.MoveNext();
				}
				else
				{
					// MSVC includes comments in any macro arguments before any expanded tokens. Save the potential first argument location in case this happens.
					TextLocation FirstArgumentLocation = Reader.TokenEndLocation;

					// Expand all the tokens
					List<Token> ExpandedTokens = new List<Token>();
					bMoveNext = ReadExpandedToken(Reader, ExpandedTokens, false, () => Reader.TokenLocation.LineIdx, new List<PreprocessorMacro>());

					// If we expanded a macro, insert all the comment tokens
					if(Reader.TokenWhitespaceLocation > FirstArgumentLocation)
					{
						TokenReader CommentReader = new TokenReader(FileName, Text, FirstArgumentLocation);
						while(CommentReader.MoveNext() && CommentReader.TokenLocation < Reader.TokenWhitespaceLocation)
						{
							string[] Comments = Text.Extract(CommentReader.TokenWhitespaceLocation, CommentReader.TokenLocation).Select(x => x.EndsWith("\\")? x.Substring(0, x.Length - 1) : x).ToArray();
							Comments[0] = Comments[0].TrimStart();
							for(int Idx = 0; Idx < Comments.Length - 1; Idx++)
							{
								Writer.WriteLine(Comments[Idx]);
							}
							Writer.Write(Comments[Comments.Length - 1].TrimEnd());
						}
					}

					// Now write the expanded tokens
					Writer.Write(Token.Format(ExpandedTokens));
				}
			}
		}
	}

	/// <summary>
	/// Tests for the preprocessor
	/// </summary>
	static class PreprocessorTests
	{
		public static void Run(bool bTestFailureCases)
		{
            RunTest("token", "token\n");
			RunTest("#", "");
			RunTest("#define X token\nX", "token\n");
			RunTest("#define X(x) token x other\nX(and one).", "token and one other.\n");
			RunTest("#define X(x,y) token x and y\nX(1, 2).", "token 1 and 2.\n");
			RunTest("#define INC(x) (x + 2)\nINC", "INC\n");
			RunTest("#define TEST(x) x\\\n?\nTEST(A)", "A?\n");
			//RunTest("%:define X token\nX", "<token>", Preprocess);

			RunTest("#define A B C D\nA", "B C D\n");
			RunTest("#define A B ## D\nA", "BD\n");
			RunTest("#define F(A) <A>\nF(x)", "<x>\n");
			RunTest("#define F(A,B) <A,B>\nF(x,y) + 1", "<x,y> + 1\n");
			RunTest("#define F(A,B,C) <A,B,C>\nF(x,y,z)", "<x,y,z>\n");
			RunTest("#define F(...) <__VA_ARGS__>\nF(x)", "<x>\n");
			RunTest("#define F(...) <__VA_ARGS__>\nF(x,y)", "<x,y>\n");
			RunTest("#define F(A,...) <A,__VA_ARGS__>\nF(x,y)", "<x,y>\n");
			RunTest("#define F(A,...) <A, __VA_ARGS__>\nF(x,y)", "<x, y>\n");
			RunTest("#define F(A,...) <A, __VA_ARGS__>\nF(x,y, z)", "<x, y, z>\n");
			
            RunTest("#define X list of tokens\nX", "list of tokens\n");
			RunTest("#define LST list\n#define TOKS tokens\n#define LOTS LST of TOKS\nLOTS LOTS", "list of tokens list of tokens\n");
			RunTest("#define LOTS LST of TOKS\n#define LST list\n#define TOKS tokens\nLOTS LOTS", "list of tokens list of tokens\n");
			RunTest("#define FUNC(x) arg=x.\nFUNC(var) FUNC(2)", "arg=var. arg=2.\n");
			RunTest("#define FUNC(x,y,z) int n = z+y*x;\nFUNC(1,2,3)", "int n = 3+2*1;\n");
			RunTest("#define X 20\n#define FUNC(x,y) x+y\nx=FUNC(X,Y);", "x=20+Y;\n");
			RunTest("#define FA(x,y) FB(x,y)\n#define FB(x,y) x + y\nFB(1,2);", "1 + 2;\n");
			RunTest("#define PRINTF(...) printf(__VA_ARGS__)\nPRINTF()", "printf()\n");
			RunTest("#define PRINTF(...) printf(__VA_ARGS__)\nPRINTF(\"hello\")", "printf(\"hello\")\n");
			RunTest("#define PRINTF(...) printf(__VA_ARGS__)\nPRINTF(\"%d\", 1)", "printf(\"%d\", 1)\n");
			RunTest("#define PRINTF(FORMAT, ...) printf(FORMAT, __VA_ARGS__)\nPRINTF(\"test\")", "printf(\"test\", )\n");
			RunTest("#define PRINTF(FORMAT, ...) printf(FORMAT, __VA_ARGS__)\nPRINTF(\"test %s\", \"hello\")", "printf(\"test %s\", \"hello\")\n");
			RunTest("#define PRINTF(FORMAT, ...) printf(FORMAT, __VA_ARGS__)\nPRINTF(\"test %s %d\", \"hello\", 1)", "printf(\"test %s %d\", \"hello\", 1)\n");
			RunTest("#define PRINTF(FORMAT, ...) printf(FORMAT, ## __VA_ARGS__)\nPRINTF(\"test\")", "printf(\"test\" )\n");
			
            RunTest("#define INC(x) (x + 2)\nINC(\n1\n)", "(1 + 2)\n");
			
            RunTest("#define STRINGIZE(ARG) #ARG\nSTRINGIZE(+=)", "\"+=\"\n");
			RunTest("#define STRINGIZE(ARG) #ARG\nSTRINGIZE(:>)", "\":>\"\n");
			RunTest("#define STRINGIZE(ARG) #ARG\nSTRINGIZE(3.1415)", "\"3.1415\"\n");

            RunTest("#define CONCAT(X, Y) X ## Y\nCONCAT(+, =)", "+=\n");
			RunTest("#define CONCAT(X, Y) X ## Y\nCONCAT(3, .14159)", "3.14159\n");
			RunTest("#define CONCAT(X, Y) X ## Y\nCONCAT(3, hello)", "3hello\n");
			RunTest("#define CONCAT(X, Y) X ## #Y\nCONCAT(u, hello)", "u\"hello\"\n");
			RunTest("#define CONCAT(X, ...) X ## __VA_ARGS__\nCONCAT(hello) there", "hello there\n");
			RunTest("#define CONCAT(X, ...) X ## __VA_ARGS__\nCONCAT(hello, there)", "hellothere\n");

			RunTest("#if 1\nfirst_branch\n#endif\nend", "first_branch\nend\n");
			RunTest("#if 0\nfirst_branch\n#endif\nend", "end\n");
			RunTest("#if 1\nbranch_1\n#else\nbranch_2\n#endif", "branch_1\n");
			RunTest("#if 0\nbranch_1\n#else\nbranch_2\n#endif", "branch_2\n");
			RunTest("#define A\n#ifdef A\nYes\n#endif", "Yes\n");
			RunTest("#define B\n#ifdef A\nYes\n#endif\nNo", "No\n");
			RunTest("#define A\n#ifndef A\nYes\n#endif\nNo", "No\n");
			RunTest("#define B\n#ifndef A\nYes\n#endif", "Yes\n");
			RunTest("#define A\n#undef A\n#ifdef A\nYes\n#endif\nNo", "No\n");
			RunTest("#define A\n#undef A\n#ifndef A\nYes\n#endif", "Yes\n");
            RunTest("#define A\n#ifdef A\nYes\n#else\nNo\n#endif", "Yes\n");
            RunTest("#define B\n#ifdef A\nYes\n#else\nNo\n#endif", "No\n");

            RunTest("#define A MACRO\nA", "MACRO\n");
			RunTest("#define A MACRO\n#undef A\nA", "A\n");
			RunTest("#define A(X) MACRO X\nA(x)", "MACRO x\n");
			RunTest("#define A(X) MACRO X\n#undef A\nA(x)", "A(x)\n");

            RunTest("#if 1 + 2 > 3\nYes\n#endif\nNo", "No\n");
			RunTest("#if 1 + 2 >= 3\nYes\n#endif", "Yes\n");
			RunTest("#define ONE 1\n#define TWO 2\n#define PLUS(x, y) x + y\n#if PLUS(ONE, TWO) > 3\nYes\n#endif\nNo", "No\n");
			RunTest("#define ONE 1\n#define TWO 2\n#define PLUS(x, y) x + y\n#if PLUS(ONE, TWO) >= 3\nYes\n#endif", "Yes\n");
			RunTest("#define ONE 1\n#if defined ONE\nOne\n#elif defined TWO\nTwo\n#else\nThree\n#endif", "One\n");
			RunTest("#define TWO 1\n#if defined ONE\nOne\n#elif defined TWO\nTwo\n#else\nThree\n#endif", "Two\n");
			RunTest("#define ONE 0\n#if defined(ONE) + defined(TWO) >= 1\nYes\n#else\nNo\n#endif", "Yes\n");
			RunTest("#define ONE 0\n#if defined(ONE) + defined(TWO) >= 2\nYes\n#else\nNo\n#endif", "No\n");
			RunTest("#define ONE 0\n#define TWO\n#if defined(ONE) + defined(TWO) >= 2\nYes\n#else\nNo\n#endif", "Yes\n");

			RunTest("#define REGISTER_NAME(num,name) NAME_##name = num,\nREGISTER_NAME(201, TRUE)", "NAME_TRUE = 201,\n");
			RunTest("#define FUNC_2(X) VALUE=X\n#define FUNC_N(X) FUNC_##X\n#define OBJECT FUNC_N(2)\nOBJECT(1234)", "VALUE=1234\n");
			RunTest("#define GCC_EXTENSION(...) 123, ## __VA_ARGS__\nGCC_EXTENSION(456)", "123,456\n");

			RunTest("#define FUNC(fmt,...) (fmt, ## __VA_ARGS__)\nFUNC(a)", "(a )\n");
			RunTest("#define FUNC(fmt,...) (fmt, ## __VA_ARGS__)\nFUNC(a,b)", "(a,b)\n");
			RunTest("#define FUNC(fmt,...) (fmt, ## __VA_ARGS__)\nFUNC(a,b )", "(a,b)\n");
			RunTest("#define FUNC(fmt, ...) (fmt, ## __VA_ARGS__)\nFUNC(a)", "(a )\n");
			RunTest("#define FUNC(fmt, ...) (fmt, ## __VA_ARGS__)\nFUNC(a,b)", "(a,b)\n");
			RunTest("#define FUNC(fmt, ...) (fmt, ## __VA_ARGS__)\nFUNC(a,b )", "(a,b)\n");

			RunTest("#define EMPTY_TOKEN\n#define FUNC(_FuncName, _HType1, _HArg1) _FuncName(_HType1 _HArg1);\nFUNC(hello, EMPTY_TOKEN, int)", "hello( int);\n");

			RunTest("#define EMPTY_TOKEN\n#define GCC_EXTENSION(...) 123 , ## __VA_ARGS__\nGCC_EXTENSION(EMPTY_TOKEN)", "123  \n");

			RunTest("#define FUNC(x) Value = x\n#define PP_JOIN(x, y) x ## y\n#define RESULT(x) PP_JOIN(FU, NC)(x)\nRESULT(1234)", "Value = 1234\n");
			RunTest("#define VALUE 1234\n#define PP_JOIN(x, y) x ## y\n#define RESULT PP_JOIN(V, ALUE)\nRESULT", "1234\n");

			RunTest("#define EMPTY_TOKEN\n#define FUNC(x) (x)\nFUNC(EMPTY_TOKEN A)", "( A)\n");
			RunTest("#define EMPTY_TOKEN\n#define FUNC(x,y) (x y)\nFUNC(EMPTY_TOKEN,A)", "( A)\n");
//			RunTest("#define EMPTY_TOKEN\n#define FUNC(x) (x)\n#define FUNC_2(x,y) FUNC(x y)\nFUNC_2(EMPTY_TOKEN,A)", "( A)\n");
//			RunTest("#define EMPTY_TOKEN\n#define FUNC(x) (x EMPTY_TOKEN)\nFUNC(A)", "(A )\n");
//			RunTest("#define EMPTY_TOKEN\n#define FUNC(x,y) (x y)\nFUNC(A,)", "(A)\n");


			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(x    y)", "(x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(x    EMPTY    y)", "(x  y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(x    EMPTY    y    EMPTY)", "(x  y )\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(EMPTY    x    EMPTY    y)", "( x  y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(    EMPTY    x    EMPTY    y)", "( x  y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(EMPTY    x    EMPTY    y    )", "( x  y)\n");

			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(x    y)", "(x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(x    EMPTY    y)", "(x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(x    EMPTY    y    EMPTY)", "(x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(EMPTY    x    EMPTY    y)", "( x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(    EMPTY    x    EMPTY    y)", "( x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(EMPTY    x    EMPTY    y    )", "( x y)\n");

			RunTest("#define EMPTY\n#define FUNC(x) ( x )\nFUNC(EMPTY EMPTY)", "(   )\n");
			RunTest("#define EMPTY\n#define FUNC(x) ( x)\nFUNC(EMPTY EMPTY x)", "(   x)\n");
			RunTest("#define EMPTY\n#define FUNC(x,y) ( x y)\n#define FUNC2(x,y) FUNC(x,y)\nFUNC2(EMPTY EMPTY EMPTY, x)", "(   x)\n");
			RunTest("#define EMPTY\n#define DOUBLE_EMPTY EMPTY EMPTY\n#define FUNC(x) ( x)\nFUNC(DOUBLE_EMPTY x)", "(   x)\n");
			RunTest("#define EMPTY\n#define DOUBLE_EMPTY EMPTY EMPTY\n#define FUNC(x,y) ( x y )\n#define FUNC2(x,y) FUNC(x,y)\nFUNC2(DOUBLE_EMPTY EMPTY, x)", "(   x )\n");
			RunTest("#define EMPTY\n#define FUNC(x,y) ( x y )\nFUNC(EMPTY EMPTY EMPTY EMPTY EMPTY EMPTY, x)", "(       x )\n");

			RunTest("#define _NON_MEMBER_CALL(FUNC, CV_REF_OPT) FUNC(X,CV_REF_OPT)\n#define _NON_MEMBER_CALL_CV(FUNC, REF_OPT) _NON_MEMBER_CALL(FUNC, REF_OPT) _NON_MEMBER_CALL(FUNC, const REF_OPT)\n#define _NON_MEMBER_CALL_CV_REF(FUNC) _NON_MEMBER_CALL_CV(FUNC, ) _NON_MEMBER_CALL_CV(FUNC, &)\n#define _IS_FUNCTION(X,CV_REF_OPT) (CV_REF_OPT)\n_NON_MEMBER_CALL_CV_REF(_IS_FUNCTION)", "() (const) (&) (const &)\n");


			RunTest("#define TEXT(x) L ## x\n#define checkf(expr, format, ...) FDebug::AssertFailed(#expr, format, ##__VA_ARGS__)\ncheckf( true, TEXT( \"hello world\" ) );", "FDebug::AssertFailed(\"true\", L\"hello world\" );\n");
			
			if (bTestFailureCases)
			{
				RunTest("#define", null);
				RunTest("#define INC(x) (x + 2)\nINC(", null);
				RunTest("#define +", null);
				RunTest("#define A B __VA_ARGS__ D\nA", null);
				RunTest("#define A ## B\nA", null);
				RunTest("#define A B ##\nA", null);
				RunTest("#define defined not_defined", null);

				RunTest("#define F(A) <A>\nF(x,y) + 1", null);
				RunTest("#define F(A,) <A>\nF(x)", null);
				RunTest("#define F(A+) <A>\nF(x)", null);
				RunTest("#define F(A,A) <A>\nF(x,y)", null);
				RunTest("#define F(A,B) <A,B>\nF(x) + 1", null);
				RunTest("#define F(A,B,) <A,B>\nF(x,y) + 1", null);
				RunTest("#define F(...) <__VA_ARGS__>\nF(x,)", null);
				RunTest("#define F(A...) <A, __VA_ARGS__>\nF(x)", null);
				RunTest("#define F(A...) <A, __VA_ARGS__>\nF(x,y)", null);
				RunTest("#define F(A,__VA_ARGS__) <A, __VA_ARGS__>\nF(x,y)", null);
				RunTest("#define F(A) #+\nF(x)", null);
				RunTest("#define F(A) #B\nF(x)", null);
				RunTest("#define F(A) ## A\nF(x)", null);
				RunTest("#define F(A) A ##\nF(x)", null);
				RunTest("#define F(A) <__VA_ARGS__>\nF(x)", null);

				RunTest("#define INC(x\n)", null);
				RunTest("#if 1\nbranch_1\n#else garbage\nbranch_2\n#endif\nend", null);
				RunTest("#if 1\nbranch_1\n#else\nbranch_2\n#endif garbage\nend", null);
				RunTest("#if 1\nbranch_1\n#else\nbranch_2\n#elif 1\nbranch_3\n#endif\nend", null);
				RunTest("#if 1\nbranch_1\n#else\nbranch_2\n#else\nbranch_3\n#endif", null);
				RunTest("#if 0\nbranch_1\n#else\nbranch_2\n#else\nbranch_3\n#endif", null);
				RunTest("#elif\nbranch_1\n#else\nbranch_2\n#endif\nend", null);
				RunTest("#ifdef +\nbranch_1\n#else\nbranch_2\n#endif", null);
				RunTest("#ifdef A 1\nbranch_1\n#else\nbranch_2\n#endif", null);
				RunTest("#define A()\n#if A(\n)\nOK\n#endif", null);
				RunTest("#define A MACRO\n#undef A B\nA", null);
			}
		}

		/// <summary>
		/// Preprocess a fragment of code, and check it results in an expected sequence of tokens
		/// </summary>
		/// <param name="Fragment">The code fragment to preprocess</param>
		/// <param name="ExpectedResult">The expected sequence of tokens, as a string. Null to indicate that the input is invalid, and an exception is expected.</param>
        static void RunTest(string Fragment, string ExpectedResult)
        {
			SourceFile File = new SourceFile(null, TextBuffer.FromString(Fragment), SourceFileFlags.Inline);

			Preprocessor Instance = new Preprocessor(null, new string[] { }, new DirectoryReference[] { });

			string Result;
			try
			{
				List<Token> OutputTokens = new List<Token>();
				foreach(PreprocessorMarkup Markup in File.Markup)
				{
					if(Markup.Type == PreprocessorMarkupType.Text)
					{
						if(Instance.IsBranchActive())
						{
							List<Token> Tokens = new List<Token>();

							TokenReader Reader = new TokenReader("", File.Text, Markup.Location);
							while(Reader.MoveNext() && Reader.TokenLocation < Markup.EndLocation)
							{
								Tokens.Add(Reader.Current);
							}

							Instance.ExpandMacros(Tokens, OutputTokens, false, 0);
						}
					}
					else
					{
						Instance.ParseMarkup(Markup.Type, Markup.Tokens, 0);
					}
				}
				Result = Token.Format(OutputTokens);
			}
			catch (PreprocessorException)
			{
				Result = null;
			}

			if((ExpectedResult == null && Result != null) || (ExpectedResult != null && Result == null) || (ExpectedResult != null && Result != null && ExpectedResult != Result))
			{
				string Prefix = "Failed test for: ";
				foreach(string Line in Fragment.Split('\n'))
				{
					Console.WriteLine("{0}{1}", Prefix, Line);
					Prefix = new string(' ', Prefix.Length);
				}

				Console.WriteLine("Expected: {0}", (ExpectedResult != null)? ExpectedResult.Replace("\n", "\\n") : "Exception");
				Console.WriteLine("Received: {0}", (Result != null)? Result.Replace("\n", "\\n") : "Exception");
				Console.WriteLine();
			}
        }

		/// <summary>
		/// Preprocess a set of source files
		/// </summary>
		/// <param name="WorkingDir">The working directory to invoke the compiler from</param>
		/// <param name="PreprocessedDir">The output directory for the preprocessed files</param>
		/// <param name="FilePairs">List of files and their compile environment</param>
		/// <param name="Log">Writer for log output</param>
		public static void WritePreprocessedFiles(DirectoryReference WorkingDir, DirectoryReference PreprocessedDir, IEnumerable<KeyValuePair<FileReference, CompileEnvironment>> FilePairs, LineBasedTextWriter Log)
		{
			DirectoryReference.CreateDirectory(PreprocessedDir);

			// Create separate directories for the compiler and include tool
			DirectoryReference CompilerDir = DirectoryReference.Combine(PreprocessedDir, "Compiler");
			DirectoryReference.CreateDirectory(CompilerDir);
			DirectoryReference CompilerNormalizedDir = DirectoryReference.Combine(PreprocessedDir, "CompilerNormalized");
			DirectoryReference.CreateDirectory(CompilerNormalizedDir);
			DirectoryReference ToolDir = DirectoryReference.Combine(PreprocessedDir, "Tool");
			DirectoryReference.CreateDirectory(ToolDir);

			// Write out the prelude file, which we can use to scrape the compiler settings
			FileReference PreludeFile = FileReference.Combine(PreprocessedDir, "Prelude.cpp");
			Preprocessor.CreatePreludeFile(PreludeFile, FilePairs.First().Value, WorkingDir);

			// Process all the files
			int MaxDegreeOfParallelism = -1;
			if(MaxDegreeOfParallelism == 1)
			{
				foreach (KeyValuePair<FileReference, CompileEnvironment> Pair in FilePairs)
				{
					WritePreprocessedFile(Pair.Key, Pair.Value, PreludeFile, WorkingDir, CompilerDir, CompilerNormalizedDir, ToolDir, Log);
				}
			}
			else
			{
				ParallelOptions Options = new ParallelOptions() { MaxDegreeOfParallelism = MaxDegreeOfParallelism };
				Parallel.ForEach(FilePairs, Options, Pair => WritePreprocessedFile(Pair.Key, Pair.Value, PreludeFile, WorkingDir, CompilerDir, CompilerNormalizedDir, ToolDir, Log));
			}
		}

		/// <summary>
		/// Preprocess a single file, with this tool and the default compiler
		/// </summary>
		/// <param name="InputFile">The source file to preprocess</param>
		/// <param name="Environment">The environment to compile with</param>
		/// <param name="PreludeFile">The prelude source file to use to scrape the compiler's default definitions</param>
		/// <param name="WorkingDir">Working directory for the compiler</param>
		/// <param name="CompilerDir">Output directory for files preprocessed by the compiler</param>
		/// <param name="CompilerNormalizedDir">Output directory for normalized versions of files preprocessed by the compiler</param>
		/// <param name="ToolDir">Output directory for files preprocessed by this tool</param>
		/// <param name="FileCache">Cache of file locations to their source file</param>
		/// <param name="Log">Output log writer</param>
		static void WritePreprocessedFile(FileReference InputFile, CompileEnvironment Environment, FileReference PreludeFile, DirectoryReference WorkingDir, DirectoryReference CompilerDir, DirectoryReference CompilerNormalizedDir, DirectoryReference ToolDir, LineBasedTextWriter Log)
		{
			BufferedTextWriter BufferedLog = new BufferedTextWriter();

			// Run the preprocessor on it
			Preprocessor ToolPreprocessor = new Preprocessor(PreludeFile, Environment.Definitions, Environment.IncludePaths);
			FileReference ToolOutputFile = FileReference.Combine(ToolDir, InputFile.GetFileNameWithoutExtension() + ".i");
			ToolPreprocessor.PreprocessFile(InputFile.FullName, ToolOutputFile, Environment.ForceIncludeFiles);

			// Generate the compiler output
			FileReference CompilerOutputFile = FileReference.Combine(CompilerDir, InputFile.GetFileNameWithoutExtension() + ".i");
			string CommandLine = String.Format("{0} {1} /P /C /utf-8 /Fi{2}", Environment.GetCommandLine(), Utility.QuoteArgument(InputFile.FullName), Utility.QuoteArgument(CompilerOutputFile.FullName));
			Utility.Run(Environment.Compiler, CommandLine, WorkingDir, BufferedLog);

			// Normalize the compiler output
			FileReference NormalizedCompilerOutputFile = FileReference.Combine(CompilerNormalizedDir, InputFile.GetFileNameWithoutExtension() + ".i");
			NormalizePreprocessedFile(ToolOutputFile, CompilerOutputFile, NormalizedCompilerOutputFile, BufferedLog);

			// Copy everything to the output log
			lock (Log)
			{
				BufferedLog.CopyTo(Log, "  ");
			}
		}

		/// <summary>
		/// Takes an output file generated by this tool, and removes any formatting differences from a compiler-generated preprocessed file.
		/// </summary>
		/// <param name="ToolFile">File preprocessed by this tool</param>
		/// <param name="CompilerFile">Path to the preprocessed file generated by the compiler</param>
		/// <param name="NormalizedCompilerFile">Output file for the normalized compiler file</param>
		/// <param name="Log"></param>
		static int NormalizePreprocessedFile(FileReference ToolFile, FileReference CompilerFile, FileReference NormalizedCompilerFile, LineBasedTextWriter Log)
		{
			// Read the tool file
			string[] ToolLines = File.ReadAllLines(ToolFile.FullName);

			// Read the compiler file
			string[] CompilerLines = File.ReadAllLines(CompilerFile.FullName);

			// Write the output file
			int NumDifferences = 0;
			using (StreamWriter Writer = new StreamWriter(NormalizedCompilerFile.FullName))
			{
				int ToolIdx = 0;
				int CompilerIdx = 0;
				while (CompilerIdx < CompilerLines.Length && ToolIdx < ToolLines.Length)
				{
					string ToolLine = ToolLines[ToolIdx];
					string CompilerLine = CompilerLines[CompilerIdx];

					// Weird inconsistency with MSVC output where it sometimes adds an extra newline at the end of the file, even if one already exists. Skip over these lines in the compiler output.
					if(CompilerLine.Length == 0 && ToolLine.Length > 0 && (CompilerIdx + 1 == CompilerLines.Length || ComparePreprocessedLines(CompilerLines[CompilerIdx + 1], ToolLine)))
					{
						CompilerIdx++;
						continue;
					}

					// If the lines are semantically equivalent, just ignore the whitespace differences and take the tool line
					if (ComparePreprocessedLines(CompilerLine, ToolLine))
					{
						Writer.WriteLine(ToolLine);
						ToolIdx++;
						CompilerIdx++;
						continue;
					}

					// Output the difference
					if(NumDifferences < 5)
					{
						Log.WriteLine("  Compiler ({0}): {1}", CompilerIdx, CompilerLine);
						Log.WriteLine("      Tool ({0}): {1}", ToolIdx, ToolLine);
						Log.WriteLine();
						NumDifferences++;
					}

					// Otherwise try to find a matching line
					int PrevCompilerIdx = CompilerIdx;
					if(!FindMatchingLines(ToolLines, ref ToolIdx, CompilerLines, ref CompilerIdx))
					{
						ToolIdx++;
						CompilerIdx++;
					}
					while(PrevCompilerIdx < CompilerIdx)
					{
						Writer.WriteLine(CompilerLines[PrevCompilerIdx]);
						PrevCompilerIdx++;
					}
				}
				while(CompilerIdx < CompilerLines.Length && (CompilerIdx < CompilerLines.Length - 1 || CompilerLines[CompilerIdx].Length > 0))
				{
					Writer.WriteLine(CompilerLines[CompilerIdx]);
					CompilerIdx++;
				}
			}
			return NumDifferences;
		}

		/// <summary>
		/// Update indices into two line lists until we find two matching lines
		/// </summary>
		/// <param name="ToolLines">Array of lines from tool output</param>
		/// <param name="ToolIdx">Index into tool lines</param>
		/// <param name="CompilerLines">Array of lines from compiler output</param>
		/// <param name="CompilerIdx">Index into compiler lines</param>
		/// <returns>True if a matching line was found</returns>
		static bool FindMatchingLines(string[] ToolLines, ref int ToolIdx, string[] CompilerLines, ref int CompilerIdx)
		{
			for(int Offset = 1; ToolIdx + Offset < ToolLines.Length || CompilerIdx + Offset < CompilerLines.Length; Offset++)
			{
				// Try to match the tool line at this offset with a compiler line at any equal or smaller offset
				if(ToolIdx + Offset < ToolLines.Length)
				{
					string ToolLine = ToolLines[ToolIdx + Offset];
					for(int OtherCompilerIdx = CompilerIdx; OtherCompilerIdx <= CompilerIdx + Offset && OtherCompilerIdx < CompilerLines.Length; OtherCompilerIdx++)
					{
						if(ComparePreprocessedLines(ToolLine, CompilerLines[OtherCompilerIdx]))
						{
							ToolIdx = ToolIdx + Offset;
							CompilerIdx = OtherCompilerIdx;
							return true;
						}
					}
				}

				// Try to match the compiler line at this offset with a tool line at any equal or smaller offset
				if (CompilerIdx + Offset < CompilerLines.Length)
				{
					string CompilerLine = CompilerLines[CompilerIdx + Offset];
					for(int OtherToolIdx = ToolIdx; OtherToolIdx <= ToolIdx + Offset && OtherToolIdx < ToolLines.Length; OtherToolIdx++)
					{
						if(ComparePreprocessedLines(ToolLines[OtherToolIdx], CompilerLine))
						{
							ToolIdx = OtherToolIdx;
							CompilerIdx = CompilerIdx + Offset;
							return true;
						}
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Compares two preprocessed source file lines, and returns true if they are the same. Ignores leading, trailing, and consecutive whitespace.
		/// </summary>
		/// <param name="First">First string to compare</param>
		/// <param name="Second">Second string to compare</param>
		/// <returns>True if the two lines are equivalent</returns>
		static bool ComparePreprocessedLines(string First, string Second)
		{
			// Skip any leading whitespace
			int FirstIdx = SkipWhitespace(First, 0);
			int SecondIdx = SkipWhitespace(Second, 0);

			// Scan to the end of the line
			while(FirstIdx < First.Length && SecondIdx < Second.Length)
			{
				if (IsWhitespace(First[FirstIdx]) && IsWhitespace(Second[SecondIdx]))
				{
					FirstIdx = SkipWhitespace(First, FirstIdx);
					SecondIdx = SkipWhitespace(Second, SecondIdx);
				}
				else if (First[FirstIdx] == Second[SecondIdx])
				{
					FirstIdx++;
					SecondIdx++;
				}
				else
				{
					return false;
				}
			}

			// Skip any trailing whitespace
			FirstIdx = SkipWhitespace(First, FirstIdx);
			SecondIdx = SkipWhitespace(Second, SecondIdx);
			return FirstIdx == First.Length && SecondIdx == Second.Length;
		}

		/// <summary>
		/// Checks whether the given character is whitespace
		/// </summary>
		/// <param name="Character">The character to check</param>
		/// <returns>True if the character is whitespace</returns>
		static bool IsWhitespace(char Character)
		{
			return Character == ' ' || Character == '\t' || Character == '\f';
		}

		/// <summary>
		/// Skips past a sequence of whitespace characters
		/// </summary>
		/// <param name="Line">The line to check</param>
		/// <param name="Idx">Current character index within the line</param>
		/// <returns>Index after the run of whitespace characters</returns>
		static int SkipWhitespace(string Line, int Idx)
		{
			while(Idx < Line.Length && IsWhitespace(Line[Idx]))
			{
				Idx++;
			}
			return Idx;
		}
	}
}
