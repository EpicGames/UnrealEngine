// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Class to allow evaluating a preprocessor expression
	/// </summary>
	class PreprocessorExpression
	{
		/// <summary>
		/// Evaluate a preprocessor expression
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <returns>Value of the expression</returns>
		public static long Evaluate(List<Token> Tokens)
		{
			int Idx = 0;
			long Result = EvaluateTernary(Tokens, ref Idx);
			if (Idx != Tokens.Count)
			{
				throw new PreprocessorException("Garbage after end of expression");
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a ternary expression (a? b :c)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateTernary(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateLogicalOr(Tokens, ref Idx);
			if(Idx < Tokens.Count && Tokens[Idx].Text == "?")
			{
				// Read the left expression
				Idx++;
				long Lhs = EvaluateTernary(Tokens, ref Idx);

				// Check for the colon in the middle
				if(Tokens[Idx].Text != ":")
				{
					throw new PreprocessorException("Expected colon for conditional operator");
				}

				// Read the right expression
				Idx++;
				long Rhs = EvaluateTernary(Tokens, ref Idx);

				// Evaluate it
				Result = (Result != 0) ? Lhs : Rhs;
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a logical-or expression (||)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateLogicalOr(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateLogicalAnd(Tokens, ref Idx);
			while(Idx < Tokens.Count && Tokens[Idx].Text == "||")
			{
				Idx++;
				long Lhs = Result;
				long Rhs = EvaluateLogicalAnd(Tokens, ref Idx);
				Result = ((Lhs != 0) || (Rhs != 0)) ? 1 : 0;
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a logical-and expression (&&)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateLogicalAnd(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateBitwiseOr(Tokens, ref Idx);
			while (Idx < Tokens.Count && Tokens[Idx].Text == "&&")
			{
				Idx++;
				long Lhs = Result;
				long Rhs = EvaluateBitwiseOr(Tokens, ref Idx);
				Result = ((Lhs != 0) && (Rhs != 0)) ? 1 : 0;
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a bitwise-OR expression (^)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateBitwiseOr(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateBitwiseXor(Tokens, ref Idx);
			while (Idx < Tokens.Count && Tokens[Idx].Text == "|")
			{
				Idx++;
				long Lhs = Result;
				long Rhs = EvaluateBitwiseXor(Tokens, ref Idx);
				Result = Lhs | Rhs;
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a bitwise-XOR expression (^)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateBitwiseXor(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateBitwiseAnd(Tokens, ref Idx);
			while (Idx < Tokens.Count && Tokens[Idx].Text == "^")
			{
				Idx++;
				long Lhs = Result;
				long Rhs = EvaluateBitwiseAnd(Tokens, ref Idx);
				Result = Lhs ^ Rhs;
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a bitwise-AND expression (&)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateBitwiseAnd(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateEquality(Tokens, ref Idx);
			while (Idx < Tokens.Count && Tokens[Idx].Text == "&")
			{
				Idx++;
				long Lhs = Result;
				long Rhs = EvaluateEquality(Tokens, ref Idx);
				Result = Lhs & Rhs;
			}
			return Result;
		}

		/// <summary>
		/// Evaluates an equality expression (==, !=)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateEquality(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateRelational(Tokens, ref Idx);
			while (Idx < Tokens.Count)
			{
				switch (Tokens[Idx].Text)
				{
					case "==":
						Idx++;
						Result = (Result == EvaluateRelational(Tokens, ref Idx)) ? 1 : 0;
						break;
					case "!=":
						Idx++;
						Result = (Result != EvaluateRelational(Tokens, ref Idx)) ? 1 : 0;
						break;
					default:
						return Result;
				}
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a relational expression (<, >, <=, >=)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateRelational(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateShift(Tokens, ref Idx);
			while (Idx < Tokens.Count)
			{
				switch (Tokens[Idx].Text)
				{
					case "<":
						Idx++;
						Result = (Result < EvaluateShift(Tokens, ref Idx))? 1 : 0;
						break;
					case ">":
						Idx++;
						Result = (Result > EvaluateShift(Tokens, ref Idx)) ? 1 : 0;
						break;
					case "<=":
						Idx++;
						Result = (Result <= EvaluateShift(Tokens, ref Idx)) ? 1 : 0;
						break;
					case ">=":
						Idx++;
						Result = (Result >= EvaluateShift(Tokens, ref Idx)) ? 1 : 0;
						break;
					default:
						return Result;
				}
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a shift expression (<<, >>)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateShift(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateAddition(Tokens, ref Idx);
			while (Idx < Tokens.Count)
			{
				switch (Tokens[Idx].Text)
				{
					case "<<":
						Idx++;
						Result = Result << (int)Math.Min(EvaluateAddition(Tokens, ref Idx), 64);
						break;
					case ">>":
						Idx++;
						Result = Result >> (int)Math.Min(EvaluateAddition(Tokens, ref Idx), 64);
						break;
					default:
						return Result;
				}
			}
			return Result;
		}

		/// <summary>
		/// Evaluates an additive expression (+, -)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateAddition(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateMultiplication(Tokens, ref Idx);
			while (Idx < Tokens.Count)
			{
				switch (Tokens[Idx].Text)
				{
					case "+":
						Idx++;
						Result = Result + EvaluateMultiplication(Tokens, ref Idx);
						break;
					case "-":
						Idx++;
						Result = Result - EvaluateMultiplication(Tokens, ref Idx);
						break;
					default:
						return Result;
				}
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a multiplicative expression (*, /, %)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateMultiplication(List<Token> Tokens, ref int Idx)
		{
			long Result = EvaluateUnary(Tokens, ref Idx);
			while(Idx < Tokens.Count)
			{
				switch (Tokens[Idx].Text)
				{
					case "*":
						Idx++;
						Result = Result * EvaluateUnary(Tokens, ref Idx);
						break;
					case "/":
						Idx++;
						Result = Result / EvaluateUnary(Tokens, ref Idx);
						break;
					case "%":
						Idx++;
						Result = Result % EvaluateUnary(Tokens, ref Idx);
						break;
					default:
						return Result;
				}
			}
			return Result;
		}

		/// <summary>
		/// Evaluates a unary expression (+, -, !, ~)
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateUnary(List<Token> Tokens, ref int Idx)
		{
			if(Idx == Tokens.Count)
			{
				throw new PreprocessorException("Early end of expression");
			}

			switch(Tokens[Idx].Text)
			{
				case "+":
					Idx++;
					return +EvaluateUnary(Tokens, ref Idx);
				case "-":
					Idx++;
					return -EvaluateUnary(Tokens, ref Idx);
				case "!":
					Idx++;
					return (EvaluateUnary(Tokens, ref Idx) == 0) ? 1 : 0;
				case "~":
					Idx++;
					return ~EvaluateUnary(Tokens, ref Idx);
				case "sizeof":
					throw new NotImplementedException();
				case "alignof":
					throw new NotImplementedException();
				case "__has_builtin":
				case "__has_extension":
				case "__has_feature":
				case "__has_warning":
				case "__is_identifier":
					if (Tokens[Idx + 1].Text != "(" || Tokens[Idx + 3].Text != ")")
					{
						throw new NotImplementedException();
					}
					Idx += 4;
					return 0;
				case "__has_c_attribute":
				case "__has_attribute":
				case "__has_declspec_attribute":
					// the argument can either be a single identifier or a scoped identifier. 
					if (Tokens[Idx + 1].Text == "(")
					{
						Idx += 3; // 3 = __has_*, (, single_identifier
						for (; Idx < Tokens.Count; ++Idx)
						{
							if (Tokens[Idx].Text == ")")
							{
								++Idx;
								return 0;
							}
						}
					}
					throw new NotImplementedException();
				case "__has_include":
				case "__has_include_next":
					if(Tokens[Idx + 1].Text == "(")
					{
						Idx += 4; // 4 = __has_inc, (, "" or <>
						for (; Idx < Tokens.Count; ++Idx)
						{
							if (Tokens[Idx].Text == ")")
							{
								++Idx;
								return 0;
							}
						}
					}
					throw new NotImplementedException();
				case "__building_module":
					if(Tokens[Idx + 1].Text != "(" || Tokens[Idx + 3].Text != ")")
					{
						throw new NotImplementedException();
					}
					Idx += 4;
					return 0;
				default:
					return EvaluatePrimary(Tokens, ref Idx);
			}
		}

		/// <summary>
		/// Evaluates a primary expression
		/// </summary>
		/// <param name="Tokens">List of tokens in the expression</param>
		/// <param name="Idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluatePrimary(List<Token> Tokens, ref int Idx)
		{
			if(Tokens[Idx].Type == TokenType.Identifier)
			{
				Idx++;
				return 0;
			}
			else if(Tokens[Idx].Type == TokenType.LeftParen)
			{
				// Read the expression
				Idx++;
				long Result = EvaluateTernary(Tokens, ref Idx);

				// Check for a closing parenthesis
				if (Tokens[Idx].Type != TokenType.RightParen)
				{
					throw new PreprocessorException("Missing closing parenthesis");
				}

				// Return the value
				Idx++;
				return Result;
			}
			else
			{
				string Token = Tokens[Idx++].Text;
				return ParseNumericLiteral(Token);
			}
		}

		/// <summary>
		/// Parse a numeric literal from the given token
		/// </summary>
		/// <param name="Token">The literal token</param>
		/// <returns>The literal value of the token</returns>
		static long ParseNumericLiteral(string Token)
		{
			// Remove any suffix from the integer type
			while(Token.EndsWith("u") || Token.EndsWith("U") || Token.EndsWith("l") || Token.EndsWith("L"))
			{
				Token = Token.Substring(0, Token.Length - 1);
			}

			// Parse the token
			long Value = 0;
			if(Token.StartsWith("0x"))
			{
				for (int Idx = 2; Idx < Token.Length; Idx++)
				{
					if (Token[Idx] >= '0' && Token[Idx] <= '9')
					{
						Value = (Value << 4) + (Token[Idx] - '0');
					}
					else if (Token[Idx] >= 'a' && Token[Idx] <= 'f')
					{
						Value = (Value << 4) + (Token[Idx] - 'a') + 10;
					}
					else if (Token[Idx] >= 'A' && Token[Idx] <= 'F')
					{
						Value = (Value << 4) + (Token[Idx] - 'A') + 10;
					}
					else
					{
						throw new PreprocessorException("Invalid hex value");
					}
				}
			}
			else if(Token.StartsWith("0"))
			{
				for (int Idx = 1; Idx < Token.Length; Idx++)
				{
					if (Token[Idx] >= '0' && Token[Idx] <= '7')
					{
						Value = (Value << 3) + (Token[Idx] - '0');
					}
					else
					{
						throw new PreprocessorException("Invalid octal value");
					}
				}
			}
			else if(Token.Length >= 2 && Token.StartsWith("'") && Token.EndsWith("'"))
			{
				if(Token.Length == 3)
				{
					Value = Token[1];
				}
				else if(Token.Length == 6 && Token[1] == '\\' && (Token[2] >= '0' && Token[2] <= '7') && (Token[3] >= '0' && Token[3] <= '7') && (Token[4] >= '0' && Token[4] <= '7'))
				{
					Value = ((Token[2] - '0') << 6) | ((Token[1] - '0') << 3) | (Token[2] - '0');
				}
				else
				{
					throw new PreprocessorException("Invalid character value");
				}
			}
			else
			{
				for (int Idx = 0; Idx < Token.Length; Idx++)
				{
					if (Token[Idx] >= '0' && Token[Idx] <= '9')
					{
						Value = (Value * 10) + (Token[Idx] - '0');
					}
					else
					{
						throw new PreprocessorException("Invalid decimal value");
					}
				}
			}
			return Value;
		}
	}

	/// <summary>
	/// Tests for preprocessor expressions
	/// </summary>
	static class PreprocessorExpressionTests
	{
		/// <summary>
		/// Run tests on expression evaluation
		/// </summary>
        public static void Run()
        {
			// Addition
            RunTest("1 + 2 > 3", 0);
			RunTest("1 + 2 >= 3", 1);

			// Equality
            RunTest("1 == 1", 1);
			RunTest("1 == 2", 0);
			RunTest("1 != 1", 0);
			RunTest("1 != 2", 1);
            RunTest("1 < 0", 0);
			RunTest("1 < 1", 0);
			RunTest("1 < 2", 1);
			RunTest("1 <= 0", 0);
			RunTest("1 <= 1", 1);
			RunTest("1 <= 2", 1);
			RunTest("1 > 0", 1);
			RunTest("1 > 1", 0);
			RunTest("1 > 2", 0);
			RunTest("1 >= 0", 1);
			RunTest("1 >= 1", 1);
			RunTest("1 >= 2", 0);
			
			// Unary operators
			RunTest("0 + 1 == +1", 1);
			RunTest("0 - 1 == -1", 1);
			RunTest("!0", 1);
			RunTest("!1", 0);
			RunTest("~0", -1L);

			// Arithmetic operators
            RunTest("3 + 7", 10);
			RunTest("3 - 7", -4);
			RunTest("3 * 7", 21);
			RunTest("21 / 3", 7);
			RunTest("24 % 7", 3);
			RunTest("2 << 4", 32);
			RunTest("32 >> 4", 2);
			RunTest("(0xab & 0xf)", 0xb);
			RunTest("(0xab | 0xf)", 0xaf);
			RunTest("(0xab ^ 0xf)", 0xa4);

			// Logical operators
			RunTest("0 || 0", 0);
			RunTest("0 || 1", 1);
			RunTest("1 || 0", 1);
			RunTest("1 || 1", 1);
			RunTest("0 && 0", 0);
			RunTest("0 && 1", 0);
			RunTest("1 && 0", 0);
			RunTest("1 && 1", 1);

			// Ternary operators
            RunTest("((0)? 123 : 456) == 456", 1);
            RunTest("((1)? 123 : 456) == 456", 0);

			// Precedence
			RunTest("3 + 27 / 3", 12);
			RunTest("(3 + 27) / 3", 10);
        }

		/// <summary>
		/// Tokenize the given expression and evaluate it, and check it matches the expected result
		/// </summary>
		/// <param name="Expression">The expression to evaluate, as a string</param>
		/// <param name="ExpectedResult">The expected value of the expression</param>
        static void RunTest(string Expression, long? ExpectedResult)
        {
			TokenReader Reader = new TokenReader("", Expression);

			List<Token> Tokens = new List<Token>();
			while (Reader.MoveNext() && Reader.Current.Text != "\n")
			{
				Tokens.Add(Reader.Current);
			}

			long? Result;
			try
			{
				Result = PreprocessorExpression.Evaluate(Tokens);
			}
			catch(PreprocessorException)
			{
				Result = null;
			}

			if(Result != ExpectedResult)
			{
				Console.WriteLine("Failed test: {0}={1} (expected {2})", Expression, Result, ExpectedResult);
			}
		}
	}
}
