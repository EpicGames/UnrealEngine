// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Possible types of source file markup
	/// </summary>
	enum PreprocessorMarkupType
	{
		/// <summary>
		/// A span of tokens which are not preprocessor directives
		/// </summary>
		Text,

		/// <summary>
		/// An #include directive
		/// </summary>
		Include,

		/// <summary>
		/// A #define directive
		/// </summary>
		Define,

		/// <summary>
		/// An #undef directive
		/// </summary>
		Undef,

		/// <summary>
		/// An #if directive
		/// </summary>
		If,

		/// <summary>
		/// An #ifdef directive
		/// </summary>
		Ifdef,

		/// <summary>
		/// An #ifndef directive
		/// </summary>
		Ifndef,

		/// <summary>
		/// An #elif directive
		/// </summary>
		Elif,

		/// <summary>
		/// An #else directive
		/// </summary>
		Else,

		/// <summary>
		/// An #endif directive
		/// </summary>
		Endif,

		/// <summary>
		/// A #pragma directive
		/// </summary>
		Pragma,

		/// <summary>
		/// An #error directive
		/// </summary>
		Error,

		/// <summary>
		/// An empty '#' on a line of its own
		/// </summary>
		Empty,

		/// <summary>
		/// Macros which decorate an include directive (eg. THIRD_PARTY_INCLUDES_START, THIRD_PARTY_INCLUDES_END)
		/// </summary>
		IncludeMarkup,

		/// <summary>
		/// Some other directive
		/// </summary>
		OtherDirective
	}

	/// <summary>
	/// Base class for an annotated section of a source file
	/// </summary>
	[Serializable]
	class PreprocessorMarkup
	{
		/// <summary>
		/// The directive corresponding to this markup
		/// </summary>
		public PreprocessorMarkupType Type;

		/// <summary>
		/// Whether this markup is within an active preprocessor condition
		/// </summary>
		public bool IsActive;

		/// <summary>
		/// The start of the annotated text
		/// </summary>
		public TextLocation Location;

		/// <summary>
		/// The end of the annotated text
		/// </summary>
		public TextLocation EndLocation;

		/// <summary>
		/// The tokens parsed for this markup. Set for directives.
		/// </summary>
		public List<Token> Tokens;

		/// <summary>
		/// For #include directives, specifies the included file that it resolves to
		/// </summary>
		public SourceFile IncludedFile;

		/// <summary>
		/// The output list of included files that this should be replaced with
		/// </summary>
		public List<SourceFile> OutputIncludedFiles;

		/// <summary>
		/// Construct the annotation with the given range
		/// </summary>
		/// <param name="Directive">The type of this directive</param>
		/// <param name="Location">The start location</param>
		/// <param name="EndLocation">The end location</param>
		/// <param name="Tokens">List of tokens</param>
		public PreprocessorMarkup(PreprocessorMarkupType Directive, TextLocation Location, TextLocation EndLocation, List<Token> Tokens)
		{
			this.Type = Directive;
			this.Location = Location;
			this.EndLocation = EndLocation;
			this.Tokens = Tokens;
		}

		/// <summary>
		/// Determines if this markup is for an inline include
		/// </summary>
		/// <returns>True if the markup is including an inline file</returns>
		public bool IsInlineInclude()
		{
			return Type == PreprocessorMarkupType.Include && IncludedFile != null && (IncludedFile.Flags & SourceFileFlags.Inline) != 0;
		}

		/// <summary>
		/// Determines if this markup indicates a preprocessor directive
		/// </summary>
		/// <returns>True if this object marks a preprocessor directive</returns>
		public bool IsPreprocessorDirective()
		{
			return Type != PreprocessorMarkupType.Text;
		}

		/// <summary>
		/// Determines if this markup indicates a conditional preprocessor directive
		/// </summary>
		/// <returns>True if this object is a conditional preprocessor directive</returns>
		public bool IsConditionalPreprocessorDirective()
		{
			switch(Type)
			{
				case PreprocessorMarkupType.If:
				case PreprocessorMarkupType.Ifdef:
				case PreprocessorMarkupType.Ifndef:
				case PreprocessorMarkupType.Elif:
				case PreprocessorMarkupType.Else:
				case PreprocessorMarkupType.Endif:
					return true;
			}
			return false;
		}

		/// <summary>
		/// How this condition modifies the condition depth. Opening "if" statements have a value of +1, "endif" statements have a value of -1, and "else" statements have a value of 0.
		/// </summary>
		public int GetConditionDepthDelta()
		{
			if(Type == PreprocessorMarkupType.If || Type == PreprocessorMarkupType.Ifdef || Type == PreprocessorMarkupType.Ifndef)
			{
				return +1;
			}
			else if(Type == PreprocessorMarkupType.Endif)
			{
				return -1;
			}
			else
			{
				return 0;
			}
		}

		/// <summary>
		/// Update the target file which an include resolves to
		/// </summary>
		/// <param name="NewTargetFile">The file being included</param>
		public bool SetIncludedFile(SourceFile NewIncludedFile)
		{
			// Process the included file, and make sure it matches the include directive
			if(IncludedFile != NewIncludedFile)
			{
				SourceFile OldIncludedFile = Interlocked.CompareExchange(ref IncludedFile, NewIncludedFile, null);
				if(OldIncludedFile == null)
				{
					OutputIncludedFiles = new List<SourceFile> { NewIncludedFile };
				}
				else if(OldIncludedFile != NewIncludedFile)
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Generate a string describing this annotation
		/// </summary>
		/// <returns>String representation for debugging</returns>
		public override string ToString()
		{
			string Result = String.Format("({0})", Location.ToString());
			if (Type != PreprocessorMarkupType.Text)
			{
				Result += ": #";
				if(Type != PreprocessorMarkupType.OtherDirective)
				{
					Result += Type.ToString().ToLower() + " ";
				}
				if(Tokens != null)
				{
					Result += Token.Format(Tokens);
				}
			}
			return Result;
		}

		/// <summary>
		/// Create markup for the given text buffer
		/// </summary>
		/// <param name="Reader">Reader for token objects</param>
		/// <returns>Array of markup objects which split up the given text buffer</returns>
		public static PreprocessorMarkup[] ParseArray(string FileName, TextBuffer Text)
		{
			TokenReader Reader = new TokenReader(FileName, Text, TextLocation.Origin);

			List<PreprocessorMarkup> Markup = new List<PreprocessorMarkup>();
			if(Reader.MoveNext())
			{
				bool bMoveNext = true;
				while(bMoveNext)
				{
					TextLocation StartLocation = Reader.TokenWhitespaceLocation;
					if (Reader.Current.Text == "#")
					{
						// Create the appropriate markup object for the directive
						PreprocessorMarkupType Type = PreprocessorMarkupType.OtherDirective;
						if(Reader.MoveNext())
						{
							switch (Reader.Current.Text)
							{
								case "include":
									Type = PreprocessorMarkupType.Include;
									break;
								case "define":
									Type = PreprocessorMarkupType.Define;
									break;
								case "undef":
									Type = PreprocessorMarkupType.Undef;
									break;
								case "if":
									Type = PreprocessorMarkupType.If;
									break;
								case "ifdef":
									Type = PreprocessorMarkupType.Ifdef;
									break;
								case "ifndef":
									Type = PreprocessorMarkupType.Ifndef;
									break;
								case "elif":
									Type = PreprocessorMarkupType.Elif;
									break;
								case "else":
									Type = PreprocessorMarkupType.Else;
									break;
								case "endif":
									Type = PreprocessorMarkupType.Endif;
									break;
								case "pragma":
									Type = PreprocessorMarkupType.Pragma;
									break;
								case "error":
									Type = PreprocessorMarkupType.Error;
									break;
								case "\n":
									Type = PreprocessorMarkupType.Empty;
									break;
							}
						}

						// Create the token list
						List<Token> Tokens = new List<Token>();
						if(Type == PreprocessorMarkupType.OtherDirective)
						{
							Tokens.Add(Reader.Current);
						}

						// Read the first token
						if(Type == PreprocessorMarkupType.Empty)
						{
							bMoveNext = true;
						}
						else if(Type == PreprocessorMarkupType.Include)
						{
							bMoveNext = Reader.MoveNext(TokenReaderContext.IncludeDirective);
						}
						else if(Type == PreprocessorMarkupType.Error)
						{
							bMoveNext = Reader.MoveNext(TokenReaderContext.TokenString);
						}
						else
						{
							bMoveNext = Reader.MoveNext();
						}

						// Read the rest of the tokens
						while(bMoveNext && Reader.Current.Text != "\n")
						{
							Tokens.Add(Reader.Current);
							bMoveNext = Reader.MoveNext();
						}

						// Create the markup
						Markup.Add(new PreprocessorMarkup(Type, StartLocation, Reader.TokenEndLocation, Tokens));

						// Move to the next token
						bMoveNext = Reader.MoveNext();
					}
					else if (Reader.Current.Text != "\n")
					{
						// Skip over as many parser tokens as possible before the next directive (or EOF)
						bMoveNext = Reader.MoveToNextLine();
						while(bMoveNext && Reader.Current.Text != "#")
						{
							bMoveNext = Reader.MoveToNextLine();
						}

						// Create the new fragment
						PreprocessorMarkupType Type = IsIncludeMarkup(FileName, Text, StartLocation, Reader.TokenLocation)? PreprocessorMarkupType.IncludeMarkup : PreprocessorMarkupType.Text;
						Markup.Add(new PreprocessorMarkup(Type, StartLocation, Reader.TokenLocation, null));
					}
					else
					{
						// Skip the empty line
						bMoveNext = Reader.MoveNext();
					}
				}
			}
			return Markup.ToArray();
		}

		/// <summary>
		/// Checks whether the given block of text is an include decoration
		/// </summary>
		/// <param name="FileName">Name of the file being parsed</param>
		/// <param name="Text">The text buffer to read from</param>
		/// <param name="StartLocation">Start of the block of text to check</param>
		/// <param name="EndLocation">End of the block of text to check</param>
		/// <returns>True if the region just consists of #include markup macros</returns>
		public static bool IsIncludeMarkup(string FileName, TextBuffer Text, TextLocation StartLocation, TextLocation EndLocation)
		{
			TokenReader Reader = new TokenReader(FileName, Text, StartLocation, EndLocation);
			while(Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				if(Reader.Current.Text == "MONOLITHIC_HEADER_BOILERPLATE" && Reader.MoveNext() && Reader.Current.Text == "(" && Reader.MoveNext() && Reader.Current.Text == ")")
				{
					continue;
				}
				if(Reader.Current.Text == "THIRD_PARTY_INCLUDES_START" || Reader.Current.Text == "THIRD_PARTY_INCLUDES_END")
				{
					continue;
				}
				if(Reader.Current.Text == "PRAGMA_DEFAULT_VISIBILITY_START" || Reader.Current.Text == "PRAGMA_DEFAULT_VISIBILITY_END")
				{
					continue;
				}
				if (Reader.Current.Text == "OPENCV_INCLUDES_START" || Reader.Current.Text == "OPENCV_INCLUDES_END")
                {
                    continue;
                }
                if(Reader.Current.Text == "PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS" || Reader.Current.Text == "PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS")
				{
					continue;
				}
                if(Reader.Current.Text == "PRAGMA_DISABLE_DEPRECATION_WARNINGS" || Reader.Current.Text == "PRAGMA_ENABLE_DEPRECATION_WARNINGS")
				{
					continue;
				}
				return false;
			}
			return true;
		}
	}
}
