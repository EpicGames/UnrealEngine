// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Reports;
using IncludeTool.Support;
using EpicGames.Core;
using UnrealBuildBase;
using System;
using System.Collections;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Operating mode for the tool
	/// </summary>
	public enum ToolMode
	{
		/// <summary>
		/// Identify all fragments and scan the workspace, but don't do anything else
		/// </summary>
		Scan,

		/// <summary>
		/// Compile all the fragments to ensure they do not have any external dependencies
		/// </summary>
		Verify,

		/// <summary>
		/// Optimize the files included by a filter
		/// </summary>
		Optimize,

		/// <summary>
		/// Gather timing data for compiling each fragment
		/// </summary>
		Timing,

		/// <summary>
		/// Compare the preprocessor output to the MSVC compiler output
		/// </summary>
		TestPreprocessor,

		/// <summary>
		/// Optimize the PCHs used by this target
		/// </summary>
		OptimizePCHs
	}

	/// <summary>
	/// The target platform to optimize
	/// </summary>
	public enum TargetPlatform
	{
		Linux,
		Win64,
	}

	/// <summary>
	/// The target configuration to optimize
	/// </summary>
	public enum TargetConfiguration
	{
		Debug,
		Development,
		Test,
		Shipping
	}

	/// <summary>
	/// Command line options for the program
	/// </summary>
	public class CommandLineOptions
	{
		/// <summary>
		/// Mode to run the tool in
		/// </summary>
		[CommandLineOption(IsRequired = true)]
		public ToolMode Mode;

		/// <summary>
		/// Input directory for source files
		/// </summary>
		[CommandLineOption]
		public string InputDir;

		/// <summary>
		/// Working directory for optimization data and reports
		/// </summary>
		[CommandLineOption(IsRequired = true)]
		public string WorkingDir;

		/// <summary>
		/// The target to scrape the compile environment from UBT for
		/// </summary>
		[CommandLineOption]
		public string Target = "UnrealPak";

		/// <summary>
		/// The target platform to scrape the compile environment for
		/// </summary>
		[CommandLineOption]
		public TargetPlatform Platform = TargetPlatform.Win64;

		/// <summary>
		/// The target configuration to scrape the compile environment for
		/// </summary>
		[CommandLineOption]
		public TargetConfiguration Configuration = TargetConfiguration.Debug;

		/// <summary>
		/// Whether to precompile the given target, or just build what's needed to run.
		/// </summary>
		[CommandLineOption]
		public bool Precompile = false;

		/// <summary>
		/// Directory for output source files
		/// </summary>
		[CommandLineOption]
		public string OutputDir;

		/// <summary>
		/// Skip generating a task list for the files to be optimized
		/// </summary>
		[CommandLineOption]
		public bool CleanTaskList;

		/// <summary>
		/// Skip checking for cycles in the source data
		/// </summary>
		[CommandLineOption]
		public bool SkipCycleCheck;

		/// <summary>
		/// Whether to suppress warnings and diagnostic information for the initial parse
		/// </summary>
		[CommandLineOption]
		public bool SkipDiagnostics;

		/// <summary>
		/// Filters the list of forward declarations in output files to the ones that the tool identifies as dependencies.
		/// </summary>
		[CommandLineOption]
		public bool RemoveForwardDeclarations;

		/// <summary>
		/// The source files from the target to preprocess and optimize
		/// </summary>
		[CommandLineOption]
		public string SourceFiles;

		/// <summary>
		/// Filter for files which should be optimized
		/// </summary>
		[CommandLineOption]
		public string OptimizeFiles;

		/// <summary>
		/// Filter for files that can be output. Every other file will be treated as read-only.
		/// </summary>
		[CommandLineOption]
		public string OutputFiles;

		/// <summary>
		/// Don't rewrite the files being optimized to make them standalone
		/// </summary>
		[CommandLineOption]
		public bool NotStandalone;

		/// <summary>
		/// Disables code which calculates a new set of includes for each output file.
		/// </summary>
		[CommandLineOption]
		public bool UseOriginalIncludes;

		/// <summary>
		/// When generating timing data, how many samples to take
		/// </summary>
		[CommandLineOption]
		public int NumSamples = 3;

		/// <summary>
		/// Number of optimization workers to spawn in parallel. Defaults to the logical number of processors.
		/// </summary>
		[CommandLineOption]
		public int MaxParallel = Environment.ProcessorCount;// / 2;

		/// <summary>
		/// When splitting up work to run in parallel, the index of the current shard
		/// </summary>
		[CommandLineOption]
		public int Shard = 1;

		/// <summary>
		/// The total number of shards being run
		/// </summary>
		[CommandLineOption]
		public int NumShards = 1;

		/// <summary>
		/// Maximum amount of time to run for. When running a sharded build, this can be used to stop and regroup dependencies before running again.
		/// </summary>
		[CommandLineOption]
		public double TimeLimit = -1.0;

		/// <summary>
		/// Proportion of files which must reference a header for it to be included in the PCH
		/// </summary>
		[CommandLineOption]
		public double PchRatio = 0.05;

		/// <summary>
		/// Whether to output a report containing complexity information
		/// </summary>
		[CommandLineOption]
		public bool Report;

		/// <summary>
		/// Whether to check out files from p4 before writing
		/// </summary>
		[CommandLineOption]
		public bool P4;

		/// <summary>
		/// Writes a verbose log per-output file
		/// </summary>
		[CommandLineOption]
		public bool VerboseFileLog;
	}

	/// <summary>
	/// The main program class
	/// </summary>
	static class Program
	{
		static readonly string[] StandardModuleDirectoryNames = { "Public", "Private", "Classes" };

		const uint SEM_FAILCRITICALERRORS  = 0x01;
		const uint SEM_NOGPFAULTERRORBOX = 0x0002;

		[DllImport("kernel32.dll")]
		static extern uint SetErrorMode(uint Mode);

		/// <summary>
		/// Main entry point for the program
		/// </summary>
		/// <param name="Args">Command line arguments</param>
		/// <returns>Exit code for the program</returns>
		static int Main(string[] Args)
		{
			// Disable the WER dialog in case the compiler crashes (this is inherited by child processes).
			SetErrorMode(SetErrorMode(0) | SEM_NOGPFAULTERRORBOX);

			// Run basic tests
			PreprocessorTests.Run(false);
			PreprocessorExpressionTests.Run();

			// Apply all the custom mutators
			foreach (Type Type in Assembly.GetExecutingAssembly().GetTypes())
			{
				if (Type.IsClass && !Type.IsAbstract && Type.IsSubclassOf(typeof(RulesMutator)))
				{
					RulesMutator Instance = (RulesMutator)Activator.CreateInstance(Type);
					Instance.Run();
				}
			}

			// Parse the command line options
			CommandLineOptions Options = new CommandLineOptions();
			if(!Support.CommandLine.Parse(Args, Options))
			{
				return 1;
			}

			// Make sure we've got an output directory
			DirectoryReference WorkingDir = new DirectoryReference(Options.WorkingDir);
			DirectoryReference.CreateDirectory(WorkingDir);

			// Create an output log
			FileReference LogFile = FileReference.Combine(WorkingDir, "Log.txt");
			using (LogWriter Log = new LogWriter(LogFile.FullName))
			{
				// Get the root directory for this branch
				DirectoryReference InputDir;
				if(Options.InputDir != null)
				{
					InputDir = new DirectoryReference(Options.InputDir);
				}
				else
				{
					InputDir = new DirectoryReference(Path.Combine(Path.GetDirectoryName(Assembly.GetExecutingAssembly().Location), "..\\..\\..\\.."));
				}

				// Generate the exported task list
				FileReference TaskListFile = FileReference.Combine(WorkingDir, "Targets", String.Format("{0} {1} {2}.xml", Options.Target, Options.Platform, Options.Configuration));
				if (Options.CleanTaskList || !FileReference.Exists(TaskListFile))
				{
					Log.WriteLine("Generating task list for {0} {1} {2}...", Options.Target, Options.Platform, Options.Configuration);
					DirectoryReference.CreateDirectory(TaskListFile.Directory);
					GenerateTaskList(InputDir, Options.Target, Options.Platform, Options.Configuration, Options.Precompile, TaskListFile, Log);
				}

				// Read the exported target information
				BuildTarget Target = BuildTarget.Read(TaskListFile.ChangeExtension(".json"));

				/// Cache all the files
				Log.WriteLine("Caching contents of {0}...", InputDir.FullName);
				Workspace.SetBranchRoot(InputDir);
				Workspace.CacheFiles(Workspace.BranchRoot);

				// Apply any additional rules to the source tree
				Rules.ApplyAdditionalRules(InputDir);

				// Reads the exported task list for the current target
				Dictionary<FileReference, CompileEnvironment> FileToCompileEnvironment;
				CompileEnvironment.ReadTaskList(TaskListFile, DirectoryReference.Combine(InputDir, "Engine", "Source"), out FileToCompileEnvironment);

				// Removed any generated.cpp files - we cannot optimize these, because they'll just be overwritten
				FileToCompileEnvironment = FileToCompileEnvironment.Where(x => !IsGeneratedCppFile(x.Key)).ToDictionary(x => x.Key, x => x.Value);

				// Read the filter for the list of input files
				FileFilter SourceFilter = CreateFilter(InputDir, Options.SourceFiles);
				if(SourceFilter != null)
				{
					Dictionary<FileReference, CompileEnvironment> NewFileToCompileEnvironment = new Dictionary<FileReference, CompileEnvironment>();
					foreach(KeyValuePair<FileReference, CompileEnvironment> Pair in FileToCompileEnvironment)
					{
						if(SourceFilter.Matches(Pair.Key.MakeRelativeTo(InputDir)))
						{
							NewFileToCompileEnvironment.Add(Pair.Key, Pair.Value);
						}
					}
					FileToCompileEnvironment = NewFileToCompileEnvironment;

					if(FileToCompileEnvironment.Count == 0)
					{
						Log.WriteLine("No files found after running source file filter");
						return 1;
					}
				}

				// Early out if we're just testing the preprocessor
				if(Options.Mode == ToolMode.TestPreprocessor)
				{
					Log.WriteLine("Writing {0} preprocessed files...", FileToCompileEnvironment.Count);
					DirectoryReference PreprocessedDir = DirectoryReference.Combine(WorkingDir, "Preprocessed");
					PreprocessorTests.WritePreprocessedFiles(WorkingDir, PreprocessedDir, FileToCompileEnvironment, Log);
					return 0;
				}

				// Find any additional system include paths for the internal preprocessor, which are normally inserted by the compiler
				List<DirectoryReference> ExtraSystemIncludePaths = new List<DirectoryReference>();
				if (Options.Platform == TargetPlatform.Linux && FileToCompileEnvironment.Count > 0)
				{
					Utility.GetSystemIncludeDirectories(FileToCompileEnvironment.First().Value, InputDir, ExtraSystemIncludePaths);
				}

				// Preprocess all the input files
				PreprocessFiles(WorkingDir, FileToCompileEnvironment.ToArray(), ExtraSystemIncludePaths, Log);

				// Find all the source files that we're interested in, and mark the precompiled headers
				Dictionary<SourceFile, CompileEnvironment> SourceFileToCompileEnvironment = FileToCompileEnvironment.ToDictionary(x => Workspace.GetFile(x.Key).SourceFile, x => x.Value);
				foreach(SourceFile SourceFile in SourceFileToCompileEnvironment.Keys)
				{
					SourceFile.Flags |= SourceFileFlags.TranslationUnit;
				}

				// Test for any include cycles
				if(!Options.SkipCycleCheck)
				{
					Log.WriteLine("Checking for circular includes...");
					List<IncludeCycle> Cycles = IncludeCycles.FindAll(SourceFileToCompileEnvironment.Keys);
					if(Cycles.Count > 0)
					{
						for(int Idx = 0; Idx < Cycles.Count; Idx++)
						{
							Log.WriteLine("Error: include cycle: {0}: {1}", Idx, Cycles[Idx].ToString());
							Log.WriteLine("Error: Unable to continue until this cycle has been removed.");
						}
						return 1;
					}
				}

				// Find all the resulting files which were preprocessed
				HashSet<SourceFile> PreprocessedFiles = new HashSet<SourceFile>();
				foreach(SourceFile File in SourceFileToCompileEnvironment.Keys)
				{
					FindPreprocessedFiles(File, PreprocessedFiles);
				}

				// Mark all the aggregate files. We don't include PCH files here because we WANT to optimize them down to nothing.
				foreach(SourceFile PreprocessedFile in PreprocessedFiles)
				{
					if(PreprocessedFile.HasHeaderGuard && (PreprocessedFile.Flags & SourceFileFlags.TranslationUnit) == 0)
					{
						if(PreprocessedFile.Markup.Skip(PreprocessedFile.BodyMinIdx).Take(PreprocessedFile.BodyMaxIdx - PreprocessedFile.BodyMinIdx).All(x => x.Type == PreprocessorMarkupType.Include))
						{
							PreprocessedFile.Flags |= SourceFileFlags.Aggregate;
						}
					}
				}

				// Find the module for each source file
				foreach (SourceFile PreprocessedFile in PreprocessedFiles)
				{
					PreprocessedFile.Module = Target.Modules.FirstOrDefault(x => PreprocessedFile.Location.IsUnderDirectory(x.Directory));
				}

				// Write a list of modules which we're optimizing, in filter format. This is useful when optimizing a large codebase module-by-module.
				using (StreamWriter FilterWriter = new StreamWriter(FileReference.Combine(WorkingDir, "Modules.txt").FullName))
				{
					HashSet<DirectoryReference> ModuleDirs = new HashSet<DirectoryReference>();
					foreach(FileReference Location in FileToCompileEnvironment.Keys.OrderBy(x => x.FullName.ToLowerInvariant()))
					{
						int PrivateIdx = Location.FullName.IndexOf("\\Private\\");
						if(PrivateIdx != -1)
						{
							DirectoryReference ModuleDir = new DirectoryReference(Location.FullName.Substring(0, PrivateIdx));
							if(ModuleDirs.Add(ModuleDir))
							{
								FilterWriter.WriteLine("/{0}/...", ModuleDir.MakeRelativeTo(InputDir).Replace('\\', '/'));
							}
						}
					}
				}

				// Find a lookup of all the files by name
				MultiValueDictionary<string, SourceFile> NameToFile = new MultiValueDictionary<string, SourceFile>(StringComparer.OrdinalIgnoreCase);
				foreach(SourceFile PreprocessedFile in PreprocessedFiles)
				{
					NameToFile.Add(PreprocessedFile.Location.GetFileName(), PreprocessedFile);
				}

				// Find the matching header file for each cpp file
				Dictionary<SourceFile, SourceFile> CppFileToHeaderFile = new Dictionary<SourceFile, SourceFile>();
				foreach(SourceFile CppFile in SourceFileToCompileEnvironment.Keys)
				{
					string HeaderName = CppFile.Location.ChangeExtension(".h").GetFileName();
					foreach(SourceFile HeaderFile in NameToFile.WithKey(HeaderName).OrderBy(x => x.Location.MakeRelativeTo(CppFile.Location.Directory).Length))
					{
						CppFileToHeaderFile[CppFile] = HeaderFile;
						break;
					}
				}

				// Find all the current include paths to start with.
				HashList<DirectoryReference> PublicIncludePaths = new HashList<DirectoryReference>();
				foreach (DirectoryReference IncludePath in FileToCompileEnvironment.Values.SelectMany(x => x.IncludePaths))
				{
					if (IncludePath.IsUnderDirectory(InputDir) && (String.Compare(IncludePath.GetDirectoryName(), "Public", true) == 0 || String.Compare(IncludePath.GetDirectoryName(), "Classes", true) == 0))
					{
						PublicIncludePaths.Add(IncludePath);
					}
				}
				PublicIncludePaths.Add(DirectoryReference.Combine(InputDir, "Engine", "Source"));

				// Find a mapping which rebases the private include paths for each module to the root private folder.
				Dictionary<DirectoryReference, DirectoryReference> RebaseIncludePaths = new Dictionary<DirectoryReference, DirectoryReference>();
				foreach(BuildModule Module in Target.Modules)
				{
					foreach(string Name in StandardModuleDirectoryNames)
					{
						DirectoryReference BaseDir = DirectoryReference.Combine(Module.Directory, Name);
						if(DirectoryReference.Exists(BaseDir))
						{
							foreach(DirectoryReference ChildDir in DirectoryReference.EnumerateDirectories(BaseDir, "*", SearchOption.AllDirectories))
							{
								RebaseIncludePaths[ChildDir] = BaseDir;
							}
						}
					}
				}

				// Find the matching private include paths for internal module includes
				Dictionary<BuildModule, HashList<DirectoryReference>> PrivateIncludePaths = new Dictionary<BuildModule, HashList<DirectoryReference>>();
				foreach(KeyValuePair<SourceFile, CompileEnvironment> Pair in SourceFileToCompileEnvironment)
				{
					if(Pair.Key.Module != null)
					{
						HashList<DirectoryReference> IncludePaths;
						if(!PrivateIncludePaths.TryGetValue(Pair.Key.Module, out IncludePaths))
						{
							IncludePaths = new HashList<DirectoryReference>(PublicIncludePaths);
							PrivateIncludePaths.Add(Pair.Key.Module, IncludePaths);
						}
						foreach(DirectoryReference IncludePath in Pair.Value.IncludePaths)
						{
							DirectoryReference RebasedIncludePath;
							if(!RebaseIncludePaths.TryGetValue(IncludePath, out RebasedIncludePath))
							{
								RebasedIncludePath = IncludePath;
							}
							IncludePaths.Add(RebasedIncludePath);
						}
						foreach(string Name in StandardModuleDirectoryNames)
						{
							DirectoryReference IncludePath = DirectoryReference.Combine(Pair.Key.Module.Directory, Name);
							if(DirectoryReference.Exists(IncludePath))
							{
								IncludePaths.Add(IncludePath);
							}
						}
					}
				}

				// Get a list of system include paths for output files
				HashList<DirectoryReference> SystemIncludePaths = new HashList<DirectoryReference>(ExtraSystemIncludePaths);
				foreach (DirectoryReference IncludePath in FileToCompileEnvironment.Values.SelectMany(x => x.IncludePaths))
				{
					if (!IncludePath.IsUnderDirectory(InputDir) && !SystemIncludePaths.Contains(IncludePath))
					{
						SystemIncludePaths.Add(IncludePath);
					}
				}

				// Create all the fragments for the source files
				Log.WriteLine("Creating fragments for {0} files...", SourceFileToCompileEnvironment.Count);
				CreateFragments(SourceFileToCompileEnvironment.Keys);

				// Build a table of all the symbols declared in the input files
				SymbolTable SymbolTable = new SymbolTable();
				foreach (SourceFile PreprocessedFile in PreprocessedFiles)
				{
					if ((PreprocessedFile.Flags & SourceFileFlags.Inline) == 0 && (PreprocessedFile.Flags & SourceFileFlags.GeneratedHeader) == 0 && PreprocessedFile.Counterpart == null)
					{
						try
						{
							SymbolTable.AddExports(PreprocessedFile);
						}
						catch(UnbalancedScopeException Ex)
						{
							Log.WriteLine("error: {0}", Ex.Message);
						}
						catch(Exception Ex)
						{
							throw new Exception(String.Format("Error while processing {0}", PreprocessedFile.Location), Ex);
						}
					}
				}

				// Figure out a list of imports for all the fragments
				foreach (SourceFile PreprocessedFile in PreprocessedFiles)
				{
					if (PreprocessedFile.Fragments != null)
					{
						foreach (SourceFragment Fragment in PreprocessedFile.Fragments)
						{
							Fragment.ReferencedSymbols = SymbolTable.FindReferences(Fragment);
						}
					}
				}

				// Only do other checks if we're not optimizing a shard
				if (!Options.SkipDiagnostics)
				{
					Log.WriteLine("Running other diagnostics...");

					// Warn about files which do not start with an include
					foreach (SourceFile SourceFile in SourceFileToCompileEnvironment.Keys)
					{
						if (SourceFile.BodyMaxIdx > SourceFile.BodyMinIdx && SourceFile.Markup[SourceFile.BodyMinIdx].Type != PreprocessorMarkupType.Include && (SourceFile.Flags & SourceFileFlags.External) == 0)
						{
							// TODO disabling for the moment
//							Log.WriteLine("warning: Source file does not begin with an #include directive: {0}", SourceFile.Location.MakeRelativeTo(InputDir));
						}
					}

					// Warn about any files that appear to be PCHs but actually include declarations
					foreach (SourceFile PreprocessedFile in PreprocessedFiles)
					{
						if(PreprocessedFile.Location.HasExtension("PCH.h") &&
						   (PreprocessedFile.Flags & SourceFileFlags.Aggregate) == 0)
						{
							Log.WriteWarning(PreprocessedFile.Location, "file appears to be PCH but also contains declarations, will not be optimized out.");
						}
					}

					// Test for any headers with old-style include guards
					foreach(SourceFile PreprocessedFile in PreprocessedFiles)
					{
						if(PreprocessedFile.HasHeaderGuard &&
						   PreprocessedFile.BodyMaxIdx < PreprocessedFile.Markup.Length &&
						   (PreprocessedFile.Flags & SourceFileFlags.External) == 0 &&
						   !Rules.IgnoreOldStyleHeaderGuards(PreprocessedFile.Location))
						{
							int LineIdx = PreprocessedFile.Markup[0].Location.LineIdx;
							Log.WriteWarning(PreprocessedFile.Location, LineIdx, $"File has old-style header guard: {PreprocessedFile.Text.Lines[LineIdx]}");
						}
					}

					// Test for any headers without include guards
					foreach(SourceFile PreprocessedFile in PreprocessedFiles)
					{
						if(!PreprocessedFile.HasHeaderGuard &&
						   (PreprocessedFile.Flags & (SourceFileFlags.TranslationUnit | SourceFileFlags.Inline | SourceFileFlags.External | SourceFileFlags.GeneratedHeader)) == 0 &&
						   PreprocessedFile.Counterpart == null &&
						   PreprocessedFile.Location.HasExtension(".h") &&
						   !PreprocessedFile.Location.GetFileName().Equals("MonolithicHeaderBoilerplate.h", StringComparison.OrdinalIgnoreCase))
						{
							Log.WriteWarning(PreprocessedFile.Location, "missing header guard");
						}
					}

					// Test for any headers that include files after the first block of code
					foreach(SourceFile PreprocessedFile in PreprocessedFiles)
					{
						if((PreprocessedFile.Flags & SourceFileFlags.TranslationUnit) == 0 &&
						   (PreprocessedFile.Flags & SourceFileFlags.AllowMultipleFragments) == 0)
						{
							int FirstTextIdx = Array.FindIndex(PreprocessedFile.Markup, x => x.Type == PreprocessorMarkupType.Text);
							if(FirstTextIdx != -1)
							{
								int LastIncludeIdx = Array.FindLastIndex(PreprocessedFile.Markup, x => x.Type == PreprocessorMarkupType.Include && x.IsActive && (x.IncludedFile.Flags & (SourceFileFlags.Pinned | SourceFileFlags.Inline)) == 0);
								if(FirstTextIdx < LastIncludeIdx)
								{
									int LineIdx = PreprocessedFile.Markup[LastIncludeIdx].Location.LineIdx;
									Log.WriteWarning(PreprocessedFile.Location, LineIdx, $"Include after first code block: {PreprocessedFile.Text.Lines[LineIdx]}");
								}
							}
						}
					}

					// Make sure headers containing forward declarations are "pure" - they do not contain any other declarations or side effects
					foreach(SourceFile PreprocessedFile in PreprocessedFiles)
					{
						if((PreprocessedFile.Flags & SourceFileFlags.FwdHeader) != 0)
						{
							if(PreprocessedFile.Fragments.Length != 1)
							{
								Log.WriteWarning(PreprocessedFile.Location, "expected only one fragment in forward declaration header");
							}
							else if(PreprocessedFile.Markup.Skip(PreprocessedFile.BodyMinIdx).Any(x => x.Type != PreprocessorMarkupType.Include && x.Type != PreprocessorMarkupType.Text))
							{
								Log.WriteWarning(PreprocessedFile.Location, "expected only include directives and text in forward declaration header");
							}
						}
					}

					// Make sure include directives only use forward slashes
					foreach (SourceFile PreprocessedFile in PreprocessedFiles)
					{
						PreprocessorMarkup[] Results = Array.FindAll(PreprocessedFile.Markup, x => x.Type == PreprocessorMarkupType.Include && x.IsActive);
						foreach (PreprocessorMarkup Result in Results)
						{
							if (Result.ToString().Contains("\\"))
							{
								Log.WriteWarning(PreprocessedFile.Location, Result.Location.LineIdx, $"Include directive using '\\' as path separator: {PreprocessedFile.Text.Lines[Result.Location.LineIdx]}");
							}
						}
					}

					// Print all the conflicting declarations
					SymbolTable.PrintConflicts(InputDir, Log);
				}

				// Build a list of forward declarations per file
				Dictionary<Symbol, SourceFile> SymbolToFwdHeader = new Dictionary<Symbol, SourceFile>();
				foreach(SourceFile PreprocessedFile in PreprocessedFiles)
				{
					if((PreprocessedFile.Flags & SourceFileFlags.FwdHeader) != 0)
					{
						if(!SymbolTable.ReadForwardDeclarations(PreprocessedFile, SymbolToFwdHeader, Log))
						{
							return 1;
						}
					}
				}

				// Compile all the fragments to check they're valid
				if(Options.Mode == ToolMode.OptimizePCHs)
				{
					Dictionary<FileReference, string> OutputFileContents = new Dictionary<FileReference, string>();
					PchReport.GeneratePCHs(Target, SourceFileToCompileEnvironment, PublicIncludePaths, SystemIncludePaths, OutputFileContents, (float)Options.PchRatio, Log);

					DirectoryReference OutputDir = new DirectoryReference(Options.OutputDir);
					WriteOutputFiles(OutputDir, OutputFileContents, Options.P4, Log);
				}
				else if (Options.Mode == ToolMode.Verify || Options.Mode == ToolMode.Optimize || Options.Mode == ToolMode.Timing)
				{
					// Write the fragments
					Log.WriteLine("Writing fragments...");
					DirectoryReference FragmentsDir = DirectoryReference.Combine(WorkingDir, "Fragments");
					WriteFragments(SourceFileToCompileEnvironment.Keys, FragmentsDir);

					// Check if we're just gathering timing data
					if(Options.Mode == ToolMode.Timing)
					{
						Log.WriteLine("Gathering timing data...");
						FragmentTimingReport.Generate(InputDir, WorkingDir, SourceFileToCompileEnvironment, Options.NumSamples, Options.Shard, Options.NumShards, Options.MaxParallel, Log);
					}
					else
					{
						// Flag all the files which should be output
						FileFilter OutputFilter = CreateFilter(InputDir, Options.OutputFiles);
						foreach(SourceFile PreprocessedFile in PreprocessedFiles)
						{
							string RelativePath = PreprocessedFile.Location.MakeRelativeTo(InputDir);
							if(OutputFilter == null || OutputFilter.Matches(RelativePath))
							{
								PreprocessedFile.Flags |= SourceFileFlags.Output;
							}
						}

						// Flag all the files which should be optimized
						FileFilter OptimizeFilter = CreateFilter(InputDir, Options.OptimizeFiles);
						foreach(SourceFile PreprocessedFile in PreprocessedFiles)
						{
							if((PreprocessedFile.Flags & SourceFileFlags.External) == 0 && (PreprocessedFile.Flags & SourceFileFlags.Aggregate) == 0 && PreprocessedFile.Counterpart == null && (PreprocessedFile.Flags & SourceFileFlags.Output) != 0)
							{
								string RelativePath = PreprocessedFile.Location.MakeRelativeTo(InputDir);
								if(OptimizeFilter == null || OptimizeFilter.Matches(RelativePath))
								{
									PreprocessedFile.Flags |= SourceFileFlags.Optimize;
								}
							}
						}

						// Set all the default dependencies for files which are not being output
						foreach(SourceFile PreprocessedFile in PreprocessedFiles)
						{
							if((PreprocessedFile.Flags & SourceFileFlags.Optimize) == 0 && PreprocessedFile.Fragments != null)
							{
								HashSet<SourceFragment> Dependencies = new HashSet<SourceFragment>();
								for(int FragmentIdx = 0; FragmentIdx < PreprocessedFile.Fragments.Length; FragmentIdx++)
								{
									SourceFragment Fragment = PreprocessedFile.Fragments[FragmentIdx];
									Dependencies.UnionWith(Fragment.FindIncludedFragments());
									Dependencies.RemoveWhere(x => x.MarkupMax <= x.MarkupMin);
									Fragment.Dependencies = new HashSet<SourceFragment>(Dependencies.Where(x => x.File.Counterpart == null));
									Dependencies.Add(Fragment);
								}
							}
						}

						// Create the intermediate directory
						DirectoryReference IntermediateDir = DirectoryReference.Combine(WorkingDir, "Intermediate");
						DirectoryReference.CreateDirectory(IntermediateDir);

						// If we're just checking fragments, do so now
						if(Options.Mode == ToolMode.Verify)
						{
							// Just compile all the fragments if running in fragments mode
							if(!ProcessSequenceTree(SequenceProbeType.Verify, SourceFileToCompileEnvironment, IntermediateDir, ExtraSystemIncludePaths, Options.MaxParallel, Options.Shard - 1, Options.NumShards, Options.TimeLimit, Log))
							{
								Log.WriteLine("Failed to compile all fragments");
								return 1;
							}
						}
						else if(Options.Mode == ToolMode.Optimize)
						{
							// If there are multiple shards, capture the timestamp of all the intermediate files before we run. We'll move the modified files to the shard output dir.
							Dictionary<FileReference, DateTime> FileToTime = new Dictionary<FileReference, DateTime>();
							if(Options.NumShards > 1)
							{
								foreach(FileReference IntermediateFile in DirectoryReference.EnumerateFiles(IntermediateDir))
								{
									DateTime LastWriteTime = new FileInfo(IntermediateFile.FullName).LastWriteTimeUtc;
									FileToTime[IntermediateFile] = LastWriteTime;
								}
							}

							// Run the dependency analyzer
							bool bResult = ProcessSequenceTree(SequenceProbeType.Optimize, SourceFileToCompileEnvironment, IntermediateDir, ExtraSystemIncludePaths, Options.MaxParallel, Options.Shard - 1, Options.NumShards, Options.TimeLimit, Log);

							// If there are multiple shards, we have to stop here. Archive everything we've touched, but don't fail until we aggregate at the end.
							if(Options.NumShards > 1)
							{
								DirectoryReference ShardDir = DirectoryReference.Combine(WorkingDir, "Shard");
								DirectoryReference.CreateDirectory(ShardDir);
								Log.WriteLine("Copying shard outputs to {0}...", ShardDir);

								foreach(FileReference IntermediateFile in DirectoryReference.EnumerateFiles(IntermediateDir))
								{
									if(IntermediateFile.HasExtension(".txt") || IntermediateFile.HasExtension(".response") || IntermediateFile.HasExtension(".state") || IntermediateFile.HasExtension(".permutation"))
									{
										DateTime CachedLastWriteTime;
										if(!FileToTime.TryGetValue(IntermediateFile, out CachedLastWriteTime) || CachedLastWriteTime != File.GetLastWriteTimeUtc(IntermediateFile.FullName))
										{
											FileReference OutputFile = FileReference.Combine(ShardDir, IntermediateFile.GetFileName());
											File.Copy(IntermediateFile.FullName, OutputFile.FullName, true);
										}
									}
								}

								return 0;
							}

							// Check the final result.
							if(!bResult)
							{
								Log.WriteLine("Failed to find dependencies");
								return 1;
							}

							// Create the output files
							Log.WriteLine("Preparing output files...");
							OutputFileBuilder.PrepareFilesForOutput(SourceFileToCompileEnvironment.Keys, CppFileToHeaderFile, SymbolToFwdHeader, !Options.NotStandalone, Options.UseOriginalIncludes, Log);

							// Write the output files
							if(Options.OutputDir != null)
							{
								// Figure out which files to output
								List<SourceFile> OutputFiles = new List<SourceFile>();
								foreach(SourceFile PreprocessedFile in PreprocessedFiles.Except(SymbolToFwdHeader.Values))
								{
									if(PreprocessedFile.Location.IsUnderDirectory(InputDir) && (PreprocessedFile.Flags & SourceFileFlags.External) == 0 && (PreprocessedFile.Flags & SourceFileFlags.GeneratedHeader) == 0 && (PreprocessedFile.Flags & SourceFileFlags.GeneratedClassesHeader) == 0 && (PreprocessedFile.Flags & SourceFileFlags.Output) != 0)
									{
										OutputFiles.Add(PreprocessedFile);
									}
								}
								OutputFiles = OutputFiles.Except(SymbolToFwdHeader.Values).ToList();

								// Write all the output files
								DirectoryReference OutputDir = new DirectoryReference(Options.OutputDir);
								WriteOptimizedFiles(InputDir, OutputDir, PublicIncludePaths, PrivateIncludePaths, SystemIncludePaths, OutputFiles, Options.P4, Options.VerboseFileLog, Options.RemoveForwardDeclarations, Log);
							}
						}
					}
				}

				if(Options.Report)
				{
					// Generate a report showing the number of preprocessed lines in each file
					FileReference ComplexityReportLocation = FileReference.Combine(WorkingDir, "Complexity.csv");
					ComplexityReport.Generate(ComplexityReportLocation, SourceFileToCompileEnvironment.Keys, Log);

					// Generate a report showing header file usage
					FileReference PchReportLocation = FileReference.Combine(WorkingDir, "PCH.csv");
					PchReport.Generate(PchReportLocation, InputDir, Target, SourceFileToCompileEnvironment, Log);

					// Generate a report showing symbol usage
					FileReference SymbolReportLocation = FileReference.Combine(WorkingDir, "Symbols.csv");
					SymbolReport.Generate(SymbolReportLocation, InputDir, PreprocessedFiles, Log);
				}
				return 0;
			}
		}

		/// <summary>
		/// Create a filter from the given settings
		/// </summary>
		/// <param name="BaseDir">Base directory</param>
		/// <param name="Rules">List of rules or rule files prefixed with @. Use - to exclude.</param>
		/// <returns>A new filter</returns>
		static FileFilter CreateFilter(DirectoryReference InputDir, string Rules)
		{
			FileFilter Filter = null;
			if(!String.IsNullOrEmpty(Rules))
			{
				foreach(string Rule in Rules.Split(';'))
				{
					string RulePath = Rule.Trim();

					// Strip the exclude filter off the start
					FileFilterType Type = FileFilterType.Include;
					if(RulePath.StartsWith("-"))
					{
						Type = FileFilterType.Exclude;
						RulePath = RulePath.Substring(1).TrimStart();
					}

					// Create the filter, using the opposite default to the first rule
					if(Filter == null)
					{
						Filter = new FileFilter((Type == FileFilterType.Include)? FileFilterType.Exclude : FileFilterType.Include);
					}

					// Add the rule to the filter, or read a text file and add its contents to the filter
					if(RulePath.StartsWith("@"))
					{
						Filter.AddRules(File.ReadAllLines(FileReference.Combine(InputDir, RulePath.Substring(1)).FullName).Select(x => x.Trim()).Where(x => !x.StartsWith(";") && x.Length > 0), Type);
					}
					else
					{
						Filter.AddRule(RulePath, Type);
					}
				}
			}
			return Filter;
		}

		/// <summary>
		/// Create a task list for compiling a given target
		/// </summary>
		/// <param name="RootDir">The root directory for the branch</param>
		/// <param name="Target">The target to compile (eg. UE4Editor Win64 Development)</param>
		/// <param name="TaskListFile">The output file for the task list</param>
		/// <param name="Log">Log to output messages to</param>
		static void GenerateTaskList(DirectoryReference RootDir, string Target, TargetPlatform Platform, TargetConfiguration Configuration, bool Precompile, FileReference TaskListFile, LineBasedTextWriter Log)
		{
			DirectoryReference IntermediateDir = DirectoryReference.Combine(RootDir, "Engine", "Intermediate", "Build");
			if(DirectoryReference.Exists(IntermediateDir))
			{
				foreach(FileReference IntermediateFile in DirectoryReference.EnumerateFiles(IntermediateDir, "UBTExport*.xml"))
				{
					FileReference.Delete(IntermediateFile);
				}
			}

			if (Utility.Run(Unreal.DotnetPath, String.Format("\"{0}\" -Mode=JsonExport {1} {2} {3}{4} -disableunity -xgeexport -nopch -nodebuginfo -define:UE_INCLUDE_TOOL=1 -execcodegenactions -outputfile=\"{5}\"",
				Unreal.UnrealBuildToolDllPath, Target, Configuration, Platform, Precompile? " -precompile" : "", TaskListFile.ChangeExtension(".json").FullName), Unreal.EngineDirectory, Log) != 0)
			{
				throw new Exception("UnrealBuildTool failed");
			}
			if (Utility.Run(Unreal.DotnetPath, String.Format("\"{0}\" {1} {2} {3}{4} -disableunity -xgeexport -nopch -nodebuginfo -define:UE_INCLUDE_TOOL=1",
				Unreal.UnrealBuildToolDllPath, Target, Configuration, Platform, Precompile? " -precompile" : ""), Unreal.EngineDirectory, Log) != 0)
			{
				throw new Exception("UnrealBuildTool failed");
			}
			DirectoryReference.CreateDirectory(TaskListFile.Directory);
			File.Copy(FileReference.Combine(RootDir, "Engine", "Intermediate", "Build", "UBTExport.000.xge.xml").FullName, TaskListFile.FullName, true);
		}

		/// <summary>
		/// Tests whether the given filename is a .generated.cpp file (or .generated.2.cpp, etc..)
		/// </summary>
		/// <param name="Location">Path to the file</param>
		/// <returns>True if the given path is to a generated cpp file</returns>
		static bool IsGeneratedCppFile(FileReference Location)
		{
			return Location.FullName.EndsWith(".cpp", StringComparison.OrdinalIgnoreCase) && Location.FullName.IndexOf(".generated.", StringComparison.OrdinalIgnoreCase) != -1;
		}

		/// <summary>
		/// Class used to record
		/// </summary>
		class MarkupState
		{
			/// <summary>
			/// Flags for each markup item in the stack
			/// </summary>
			public BitArray Flags;

			/// <summary>
			/// Include stack which was used to create these flags
			/// </summary>
			public FileReference[] IncludeStack;
		}

		/// <summary>
		/// Preprocess the given files and update the markup on them to reflect which directives are active, and which files #include directives resolve to.
		/// </summary>
		/// <param name="WorkingDir">The working directory for fragments</param>
		/// <param name="FilesToCompile">Array of files to compile, with their corresponding compile environment</param>
		static void PreprocessFiles(DirectoryReference WorkingDir, KeyValuePair<FileReference, CompileEnvironment>[] FilesToCompile, IEnumerable<DirectoryReference> ExtraSystemIncludePaths, LineBasedTextWriter Log)
		{
			if(FilesToCompile.Length > 0)
			{
				// Generate the prelude file
				FileReference PreludeFile = FileReference.Combine(WorkingDir, "Prelude.cpp");
				Preprocessor.CreatePreludeFile(PreludeFile, FilesToCompile[0].Value, WorkingDir);

				// Create a mapping from source file to whether each markup item is active or not
				ConcurrentDictionary<SourceFile, MarkupState> FileToActiveMarkup = new ConcurrentDictionary<SourceFile, MarkupState>();
				Utility.ParallelForWithStatus("Parsing markup...", 0, FilesToCompile.Length, Idx => PreprocessFile(FilesToCompile[Idx].Key, PreludeFile, FilesToCompile[Idx].Value, ExtraSystemIncludePaths, FileToActiveMarkup, Log), Log);
			}
		}

		/// <summary>
		/// Preprocess a given file, updating the markup for each included file
		/// </summary>
		/// <param name="Location">The file to process</param>
		/// <param name="PreludeLocation">Built-in definitions for the preprocessor, scraped from the underlying compiler</param>
		/// <param name="CompileEnvironment">Compile environment for the given file</param>
		/// <param name="FileToActiveMarkup">Map from source file to bit array of the markup blocks that are active. Used to verify all preprocessed files parse the same way.</param>
		/// <param name="Log">Writer for output messages</param>
		static void PreprocessFile(FileReference Location, FileReference PreludeLocation, CompileEnvironment CompileEnvironment, IEnumerable<DirectoryReference> ExtraSystemIncludePaths, ConcurrentDictionary<SourceFile, MarkupState> FileToActiveMarkup, LineBasedTextWriter Log)
		{
			try
			{
				// Get the initial file
				PreprocessorFile InitialFile = new PreprocessorFile(Location.FullName, Workspace.GetFile(Location));

				// Get all the include paths. For Clang, we have to add these paths manually.
				List<DirectoryReference> IncludePaths = new List<DirectoryReference>(CompileEnvironment.IncludePaths);
				IncludePaths.AddRange(ExtraSystemIncludePaths);

				// Set up the preprocessor with the correct defines for this environment
				Preprocessor PreprocessorInst = new Preprocessor(PreludeLocation, CompileEnvironment.Definitions, IncludePaths);
				foreach (FileReference ForceIncludedFile in CompileEnvironment.ForceIncludeFiles)
				{
					PreprocessIncludedFile(PreprocessorInst, new PreprocessorFile(ForceIncludedFile.FullName, Workspace.GetFile(ForceIncludedFile)), FileToActiveMarkup, Log);
				}
				PreprocessIncludedFile(PreprocessorInst, InitialFile, FileToActiveMarkup, Log);
			}
			catch (TokenException ex)
			{
				Log.WriteLine(ex.Message);
			}
			catch (Exception ex)
			{
				Log.WriteLine("Error parsing {0}: {1}", Location, ex.ToString());
			}
		}

		/// <summary>
		/// Process an included file with the given preprocessor
		/// </summary>
		/// <param name="Preprocessor">The current preprocessor state</param>
		/// <param name="IncludedFile">The file to include</param>
		/// <param name="FileToActiveMarkup">Map from source file to bit array of the markup blocks that are active. Used to verify all preprocessed files parse the same way.</param>
		/// <param name="Log">Writer for output messages</param>
		static void PreprocessIncludedFile(Preprocessor Preprocessor, PreprocessorFile IncludedFile, ConcurrentDictionary<SourceFile, MarkupState> FileToActiveMarkup, LineBasedTextWriter Log)
		{
			if(Preprocessor.PushFile(IncludedFile))
			{
				// Try to an existing source file at this location, or create a new one
				SourceFile File = IncludedFile.WorkspaceFile.ReadSourceFile();

				// Create a record to store which markup is active for this file
				MarkupState ActiveMarkup = new MarkupState();
				ActiveMarkup.Flags = new BitArray(File.Markup.Length);
				ActiveMarkup.IncludeStack = Preprocessor.CaptureIncludeStack();

				// Create a BitArray to store the active markup for this file
				for(int Idx = 0; Idx < File.Markup.Length; Idx++)
				{
					// Save the active state
					ActiveMarkup.Flags[Idx] = Preprocessor.IsBranchActive();

					// Process the directive
					PreprocessorMarkup Markup = File.Markup[Idx];
					try
					{
						if (Markup.Type != PreprocessorMarkupType.Include)
						{
							Preprocessor.ParseMarkup(Markup.Type, Markup.Tokens, Markup.Location.LineIdx);
						}
						else
						{
							if (Markup.Tokens.Count == 1 && (Markup.Tokens[0].Type == TokenType.StringLiteral || Markup.Tokens[0].Type == TokenType.SystemInclude) && Markup.Tokens[0].Text.Contains("//"))
							{
								Log.WriteLine("{0}({1}): warning: include path has multiple slashes ({2})", File.Location, Markup.Location.LineIdx + 1, Markup.Tokens[0].Text);
							}

							if (Preprocessor.IsBranchActive())
							{
								// Figure out which file is being included
								PreprocessorFile TargetFile = Preprocessor.ResolveInclude(Markup.Tokens, Markup.Location.LineIdx);
								if (!Markup.SetIncludedFile(TargetFile.WorkspaceFile.ReadSourceFile()))
								{
									PreprocessorFile ExistingTargetFile = new PreprocessorFile(TargetFile.FileName, Workspace.GetFile(Markup.IncludedFile.Location));
									Log.WriteLine("{0}({1}): warning: include text resolved to different locations", File.Location, Markup.Location.LineIdx + 1);
									Log.WriteLine("    First: {0}", ExistingTargetFile.Location);
									Log.WriteLine("    Now:   {0}", TargetFile.Location);
									TargetFile = ExistingTargetFile;
								}
								PreprocessIncludedFile(Preprocessor, TargetFile, FileToActiveMarkup, Log);
								ActiveMarkup.Flags[Idx] = true;
							}
						}
					}
					catch(PreprocessorException Ex)
					{
						Log.WriteLine("{0}({1}): error: {2}", File.Location, Markup.Location.LineIdx + 1, Ex.Message);
						return;
					}
				}

				// Make sure that external files only include other external files
				if((File.Flags & SourceFileFlags.External) != 0)
				{
					foreach(PreprocessorMarkup Markup in File.Markup)
					{
						if (Markup.IncludedFile != null && (Markup.IncludedFile.Flags & SourceFileFlags.External) == 0
							&& !File.Location.GetFileName().Contains("FramePro")
							&& !File.Location.GetFileName().Contains("Recast")
							&& !File.Location.FullName.Contains("libSampleRate")
							&& !File.Location.FullName.Contains("libav")
							&& !File.Location.FullName.Contains("lz4")
							&& !File.Location.FullName.Contains("NNE")
							&& !File.Location.FullName.Contains("PhyaLib")
							&& !File.Location.FullName.Contains("RiderLink")
							)
						{
							Log.WriteLine("{0}({1}): warning: external file is including non-external file ('{2}')", IncludedFile.Location.FullName, Markup.Location.LineIdx + 1, Markup.IncludedFile.Location);
						}
					}
				}

				// Set the active markup for this source file.
				if((File.Flags & SourceFileFlags.Inline) == 0)
				{
					MarkupState CurrentActiveMarkup = FileToActiveMarkup.GetOrAdd(File, ActiveMarkup);
					if(CurrentActiveMarkup == ActiveMarkup)
					{
						// We were the first thread to update the markup for this file; update the flags on the individual markup objects.
						for(int Idx = 0; Idx < File.Markup.Length; Idx++)
						{
							if(ActiveMarkup.Flags[Idx])
							{
								File.Markup[Idx].IsActive = true;
							}
						}
					}
					else if((File.Flags & SourceFileFlags.External) == 0)
					{
						bool bIgnoreInconsistentState = false;
						int IgnoreInconsistentStateIdx = -1;

						// Another thread has already derived the active markup for this file; check it's identical.
						for(int Idx = 0; Idx < File.Markup.Length; Idx++)
						{
							if (File.Markup[Idx].Type == PreprocessorMarkupType.Define && File.Markup[Idx].Tokens[0].Text == "UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE")
							{
								IgnoreInconsistentStateIdx = Idx;
								bIgnoreInconsistentState = true;
							}

							if (bIgnoreInconsistentState)
							{
								if (File.Markup[Idx].Type == PreprocessorMarkupType.Undef && File.Markup[Idx].Tokens[0].Text == "UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE")
								{
									IgnoreInconsistentStateIdx = -1;
									bIgnoreInconsistentState = false;
								}

								continue;
							}

							if(ActiveMarkup.Flags[Idx] != CurrentActiveMarkup.Flags[Idx] && !Rules.AllowDifferentMarkup(File, Idx))
							{
								lock(Log)
								{
									Log.WriteLine("{0}({1}): warning: Inconsistent preprocessor state. Every block of code in a header file should consistently be active or inactive regardless of where it is included from. Inconsistencies prevent IWYU optimization, and may be indicative of programming errors.", File.Location.FullName, File.Markup[Idx].Location.LineIdx + 1);
									Log.WriteLine("    First include stack ({0}):", CurrentActiveMarkup.Flags[Idx]? "code is active" : "code is inactive");
									foreach(FileReference IncludeFile in CurrentActiveMarkup.IncludeStack.Reverse())
									{
										Log.WriteLine("        {0}", IncludeFile);
									}
									Log.WriteLine("    Second include stack ({0}):", ActiveMarkup.Flags[Idx]? "code is active" : "code is inactive");
									foreach(FileReference IncludeFile in ActiveMarkup.IncludeStack.Reverse())
									{
										Log.WriteLine("        {0}", IncludeFile);
									}
								}
							}
						}

						if (bIgnoreInconsistentState)
						{
							Log.WriteLine($"{File.Location.FullName}({File.Markup[IgnoreInconsistentStateIdx].Location.LineIdx + 1}): warning: UE_INCLUDE_IGNORE_INCONSISTENT_STATE was defined, but not undefined. Add '#undef UE_INCLUDE_IGNORE_INCONSISTENT_STATE' to this file.");
						}
					}
				}

				// Remove the file from the stack
				Preprocessor.PopFile();
			}
		}

		/// <summary>
		/// Find all the preprocessed files included by the given file
		/// </summary>
		/// <param name="File">The file to search</param>
		/// <param name="PreprocessedFiles">All the preprocessed files</param>
		static void FindPreprocessedFiles(SourceFile File, HashSet<SourceFile> PreprocessedFiles)
		{
			if(PreprocessedFiles.Add(File))
			{
				foreach(PreprocessorMarkup Markup in File.Markup)
				{
					if(Markup.Type == PreprocessorMarkupType.Include && Markup.IsActive)
					{
						FindPreprocessedFiles(Markup.IncludedFile, PreprocessedFiles);
					}
				}
			}
		}

		/// <summary>
		/// Creates all the fragments for files in the given list, and the files they reference
		/// </summary>
		/// <param name="Files">Array of files to create fragments for</param>
		static void CreateFragments(IEnumerable<SourceFile> Files)
		{
			HashSet<string> UniqueNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			foreach(SourceFile File in Files)
			{
				CreateFragmentsRecursively(File, UniqueNames);
			}
		}

		/// <summary>
		/// Recurses through all the files included by a given file, creating markup for them as necessary
		/// </summary>
		/// <param name="File">File to recurse through</param>
		static void CreateFragmentsRecursively(SourceFile File, HashSet<string> UniqueNames)
		{
			if(File.Fragments == null)
			{
				// Generate the list of fragments for this file
				if((File.Flags & SourceFileFlags.External) != 0 || (File.Flags & SourceFileFlags.Inline) != 0)
				{
					// External and inline files can just consist of one giant fragment
					SourceFragment Fragment = new SourceFragment(File, 0, 0);
					Fragment.Location = File.Location;
					File.Fragments = new SourceFragment[] { Fragment };
				}
				else
				{
					// Build a new fragment for each span between #include blocks
					List<SourceFragment> Fragments = new List<SourceFragment>();
					for(int Idx = File.BodyMinIdx; Idx < File.BodyMaxIdx; )
					{
						int FragmentMinIdx = Idx;

						// Skip past all the includes at the start of this fragment. Contextual includes have to be part of the body, so stop as soon as we see one.
						for(; Idx < File.BodyMaxIdx; Idx++)
						{
							PreprocessorMarkup Markup = File.Markup[Idx];
							if(!Markup.IsConditionalPreprocessorDirective())
							{
								if(Markup.Type != PreprocessorMarkupType.Include || Markup.IsInlineInclude())
								{
									break;
								}
							}
						}

						// Scan until the end of the fragment
						for(; Idx < File.BodyMaxIdx; Idx++)
						{
							PreprocessorMarkup Markup = File.Markup[Idx];
							if(Markup.Type == PreprocessorMarkupType.Include && Markup.IsActive && !Markup.IsInlineInclude())
							{
								break;
							}
						}

						// Create the fragment
						Fragments.Add(new SourceFragment(File, FragmentMinIdx, Idx));
					}
					File.Fragments = Fragments.ToArray();

					// Get a unique name for this file
					string UniqueName = File.Location.GetFileName();
					for(int Idx = 1; !UniqueNames.Add(UniqueName); Idx++)
					{
						UniqueName = String.Format("{0}_{1}", File.Location.GetFileName(), Idx);
					}

					// Assign a unique name to each fragment
					for(int Idx = 0; Idx < File.Fragments.Length; Idx++)
					{
						string FragmentBaseName = Path.GetFileNameWithoutExtension(UniqueName);
						if(File.Fragments.Length > 1)
						{
							FragmentBaseName += String.Format(".{0}of{1}", Idx + 1, File.Fragments.Length);
						}
						File.Fragments[Idx].UniqueName = FragmentBaseName + Path.GetExtension(UniqueName);
					}

					// Generate fragments for everything it includes
					foreach(PreprocessorMarkup Markup in File.Markup)
					{
						if(Markup.Type == PreprocessorMarkupType.Include && Markup.IsActive)
						{
							CreateFragmentsRecursively(Markup.IncludedFile, UniqueNames);
						}
					}
				}
			}
		}

		/// <summary>
		/// Write out all the fragments for the given files
		/// </summary>
		/// <param name="Files">The files to write fragments for</param>
		static void WriteFragments(IEnumerable<SourceFile> Files, DirectoryReference OutputDir)
		{
			// Create the output directory
			DirectoryReference.CreateDirectory(OutputDir);

			// Assign names to all the fragments
			HashSet<SourceFile> VisitedFiles = new HashSet<SourceFile>();
			foreach(SourceFile File in Files)
			{
				WriteFragmentsRecursively(File, OutputDir, VisitedFiles);
			}
		}

		/// <summary>
		/// Recursively write out the fragments for a file
		/// </summary>
		/// <param name="File">The current file</param>
		/// <param name="OutputDir">Output directory for the new fragments</param>
		/// <param name="VisitedFiles">Set of files which have been visited</param>
		static void WriteFragmentsRecursively(SourceFile File, DirectoryReference OutputDir, HashSet<SourceFile> VisitedFiles)
		{
			// Skip files we've already visited, and external or inline files (which we don't write any fragments for)
			if(VisitedFiles.Add(File) && (File.Flags & SourceFileFlags.External) == 0 && (File.Flags & SourceFileFlags.Inline) == 0)
			{
				// Write all the included fragments
				foreach(PreprocessorMarkup Markup in File.Markup)
				{
					if(Markup.Type == PreprocessorMarkupType.Include && Markup.IncludedFile != null)
					{
						WriteFragmentsRecursively(Markup.IncludedFile, OutputDir, VisitedFiles);
					}
				}

				// Write all the fragments
				foreach(SourceFragment Fragment in File.Fragments)
				{
					// Get the location of this fragment
					Fragment.Location = FileReference.Combine(OutputDir, Fragment.UniqueName);

					// Write the fragment to a string buffer (using '\n' as the newline separator, rather than \r\n)
					using (StreamWriter Writer = new StreamWriter(Fragment.Location.FullName))
					{
						if((File.Flags & SourceFileFlags.External) != 0)
						{
							Writer.WriteLine("#include \"{0}\"", File.Location.FullName);
						}
						else
						{
							Fragment.Write(Writer);
						}
					}

					// Compute the hash
					Fragment.Digest = Utility.ComputeDigest(Fragment.Location);
				}
			}
		}

		/// <summary>
		/// Find dependencies for files in the given dependency tree
		/// </summary>
		/// <param name="Type">Type of probes to create</param>
		/// <param name="SourceFileToCompileEnvironment">Source files and their corresponding compile environment</param>
		/// <param name="IntermediateDir">Output directory for intermediate files</param>
		/// <param name="MaxParallel">Maximum number of tasks to run in parallel.</param>
		/// <param name="bMultiProcess">Whether to spawn child processes to execute the search. Useful with XGE.</param>
		/// <param name="Log">Writer for log files</param>
		/// <returns>True on success</returns>
		public static bool ProcessSequenceTree(SequenceProbeType Type, Dictionary<SourceFile, CompileEnvironment> SourceFileToCompileEnvironment, DirectoryReference IntermediateDir, IEnumerable<DirectoryReference> ExtraSystemIncludePaths, int MaxParallel, int ShardIdx, int NumShards, double TimeLimit, LineBasedTextWriter Log)
		{
			Stopwatch Timer = Stopwatch.StartNew();
			Log.WriteLine("Creating probes...");

			// Create a map of fragment to the probe created to build it
			Dictionary<SourceFragment, Lazy<SequenceProbe>> FragmentToLazyProbe = new Dictionary<SourceFragment, Lazy<SequenceProbe>>();

			// Create all the unique probes
			HashSet<string> UniqueProbeNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			foreach(SourceFile SourceFile in SourceFileToCompileEnvironment.Keys)
			{
				// Flatten the file into a list of fragments
				List<SourceFragment> Fragments = new List<SourceFragment>();
				List<Tuple<int, SourceFile>> IncludeHistory = new List<Tuple<int, SourceFile>>();
				SourceFile.FindIncludedFragments(Fragments, IncludeHistory, new HashSet<SourceFile>());

				// Create a probe for each fragment we haven't seen before
				for(int FragmentIdx = 0; FragmentIdx < Fragments.Count; FragmentIdx++)
				{
					SourceFragment Fragment = Fragments[FragmentIdx];
					if (Fragment.Dependencies == null && !FragmentToLazyProbe.ContainsKey(Fragment))
					{
						// Get a unique name for the worker
						string UniqueName = Fragment.Location.GetFileName();
						for (int Idx = 2; !UniqueProbeNames.Add(UniqueName); Idx++)
						{
							UniqueName = String.Format("{0}_{1}{2}", Fragment.Location.GetFileNameWithoutExtension(), Idx, Fragment.Location.GetExtension());
						}

						// Add it to the dictionary
						Lazy<SequenceProbe> LazyProbe = null;
						if (NumShards == 1 || ((uint)String.Join("\n", Fragment.File.Text.Lines).GetHashCode() % (uint)NumShards) == ShardIdx)
						{
							SourceFragment[] CurrentFragments = Fragments.Take(FragmentIdx + 1).ToArray();
							Tuple<int, SourceFile>[] CurrentIncludeHistory = IncludeHistory.Where(x => x.Item1 < FragmentIdx).ToArray();
							CompileEnvironment CurrentCompileEnvironment = SourceFileToCompileEnvironment[SourceFile];
							LazyProbe = new Lazy<SequenceProbe>(() => new SequenceProbe(Type, CurrentFragments, CurrentIncludeHistory, CurrentCompileEnvironment, IntermediateDir, ExtraSystemIncludePaths, UniqueName));
						}
						FragmentToLazyProbe.Add(Fragment, LazyProbe);
					}
				}
			}

			// Remove any invalid probes
			FragmentToLazyProbe = FragmentToLazyProbe.Where(x => x.Value != null).ToDictionary(x => x.Key, x => x.Value);

			// Construct them on different threads
			List<Lazy<SequenceProbe>> ProbesToConstruct = FragmentToLazyProbe.Where(x => x.Value != null).Select(x => x.Value).ToList();
			Utility.ParallelForWithStatus("Creating probes...", 0, ProbesToConstruct.Count, x => { var y = ProbesToConstruct[x].Value; }, Log);

			// Create a dictionary with all the created probes
			Dictionary<SourceFragment, SequenceProbe> FragmentToProbe = FragmentToLazyProbe.Where(x => x.Value != null).ToDictionary(x => x.Key, x => x.Value.Value);

			// Queue of workers which are ready to run
			Queue<SequenceProbe> QueuedProbes = new Queue<SequenceProbe>();
			foreach(SequenceProbe Probe in FragmentToProbe.Values)
			{
				if(Type == SequenceProbeType.Verify || !Probe.IsComplete)
				{
					QueuedProbes.Enqueue(Probe);
				}
			}

			// Keep track of the number of failures and successes
			int NumFailed = 0;
			int NumSucceeded = 0;

			// Create the work queue outside a try/finally block, to clean them up on abnormal termination
			List<SequenceProbe> ActiveProbes = new List<SequenceProbe>();
			try
			{
				// Process the queue
				DateTime NextUpdateTime = DateTime.MinValue;
				for(;;)
				{
					// Start as many probes as possible
					while(ActiveProbes.Count < MaxParallel && QueuedProbes.Count > 0)
					{
						SequenceProbe Probe = QueuedProbes.Dequeue();
						Probe.Start(Log);
						ActiveProbes.Add(Probe);
					}

					// Print a progress update
					DateTime CurrentTime = DateTime.UtcNow;
					if(CurrentTime > NextUpdateTime || ActiveProbes.Count == 0)
					{
						if(Type == SequenceProbeType.Verify)
						{
							// Just show the number of probes we've processed
							int Progress = (40 * (NumFailed + NumSucceeded)) / FragmentToProbe.Count;
							string ProgressBar = new string('.', Progress) + new string(' ', 40 - Progress);
							Log.WriteLine("[{0}] Verify <{1}> {2}/{3} probes, {4} failed, {5} active ({6}%)", Timer.Elapsed.ToString(@"hh\:mm\:ss"), ProgressBar, NumFailed + NumSucceeded, FragmentToProbe.Count, NumFailed, ActiveProbes.Count, (FragmentToProbe.Count == 0)? 100 : ((NumFailed + NumSucceeded) * 100) / FragmentToProbe.Count);
						}
						else if(Type == SequenceProbeType.Optimize)
						{
							// Count the total number of fragments left to process
							int TotalFragments = FragmentToProbe.Values.Sum(x => x.FragmentCount);

							// Count the number of remaining fragments
							int NumCompletedFragments = FragmentToProbe.Values.Sum(x => x.CompletedFragmentCount);

							// Count the number of completed probes
							int NumCompletedProbes = FragmentToProbe.Values.Count(x => x.IsComplete);

							// Count the number of fragments currently being searched
							char[] Progress = new char[40];
							for(int Idx = 0; Idx < Progress.Length; Idx++)
							{
								Progress[Idx] = ' ';
							}
							for(int Idx = 0; Idx < ((TotalFragments == 0)? Progress.Length : (NumCompletedFragments * Progress.Length) / TotalFragments); Idx++)
							{
								Progress[Idx] = '.';
							}
							Log.WriteLine("[{0}] Optimize <{1}> {2}/{3} probes, {4}{5} active, {6}/{7} fragments ({8}%)", Timer.Elapsed.ToString(@"hh\:mm\:ss"), new string(Progress), NumCompletedProbes, FragmentToProbe.Count, (NumFailed == 0)? "" : String.Format("{0} failed, ", NumFailed), ActiveProbes.Count, NumCompletedFragments, TotalFragments, (TotalFragments == 0)? 100 : (NumCompletedFragments * 100) / TotalFragments);

							// Check the time limit
							if(TimeLimit > 0 && Timer.Elapsed.TotalHours > TimeLimit)
							{
								Log.WriteLine("Halting due to expired time limit.");
								break;
							}
						}
						else
						{
							throw new NotImplementedException();
						}
						NextUpdateTime = CurrentTime + TimeSpan.FromSeconds(30);
					}

					// Quit if there's nothing left to do
					if(ActiveProbes.Count == 0)
					{
						break;
					}

					// Wait for any process to finish, or the next time to update the output
					int ProbeIndex = SequenceProbe.WaitForAny(ActiveProbes.ToArray(), (int)(NextUpdateTime - CurrentTime).TotalMilliseconds + 1);
					if(ProbeIndex != -1)
					{
						// Get the completed probe
						SequenceProbe CompletedProbe = ActiveProbes[ProbeIndex];

						// Decide what to do next
						SequenceProbeResult Result = CompletedProbe.Join(Log);
						if(Result == SequenceProbeResult.Failed)
						{
							Log.WriteLine("Failed to compile '{0}'", CompletedProbe.LastFragment.Location.ToString());
							ActiveProbes.RemoveAt(ProbeIndex);
							NumFailed++;
						}
						else if(Result == SequenceProbeResult.FailedAllowRetry)
						{
							// Add it to the back of the queue
							ActiveProbes.RemoveAt(ProbeIndex);
							QueuedProbes.Enqueue(CompletedProbe);
						}
						else if(Result == SequenceProbeResult.Updated)
						{
							// Run it again
							CompletedProbe.Start(Log);
						}
						else if(Result == SequenceProbeResult.Completed)
						{
							// Remove the worker
							ActiveProbes.RemoveAt(ProbeIndex);
							NumSucceeded++;
						}
						else
						{
							throw new Exception("Unhandled return code from DependencyWorker.Join()");
						}
					}
				}
			}
			finally
			{
				// Kill everything off
				foreach(SequenceProbe Probe in ActiveProbes)
				{
					Probe.Dispose();
				}
			}

			// Now that everything's finished, fixup every fragment to be a superset of all its dependencies' dependencies.
			if (Type == SequenceProbeType.Optimize)
			{
				HashSet<SourceFragment> VisitedFragments = new HashSet<SourceFragment>();
				foreach(SourceFragment LastFragment in FragmentToProbe.Keys)
				{
					FixupDependencies(LastFragment, VisitedFragments);
				}
			}

			// Return whether everything succeeded
			return NumFailed == 0;
		}

		/// <summary>
		/// Fixup all the dependencies of the given fragment to ensure it includes all its dependencies recursively
		/// </summary>
		/// <param name="Fragment">The fragment to fixup dependencies for</param>
		/// <param name="VisitedFragments">Set of visited fragments so far</param>
		static void FixupDependencies(SourceFragment Fragment, HashSet<SourceFragment> VisitedFragments)
		{
			if(VisitedFragments.Add(Fragment) && Fragment.Dependencies != null)
			{
				SourceFragment[] Dependencies = Fragment.Dependencies.ToArray();
				foreach(SourceFragment Dependency in Dependencies)
				{
					FixupDependencies(Dependency, VisitedFragments);
				}
				foreach(SourceFragment Dependency in Dependencies)
				{
					if(Dependency.Dependencies != null)
					{
						Fragment.Dependencies.UnionWith(Dependency.Dependencies);
					}
				}
			}
		}

		/// <summary>
		/// Write all the output files
		/// </summary>
		/// <param name="InputDir">Directory containing all the input files</param>
		/// <param name="OutputDir">The directory to write all the output files to</param>
		/// <param name="IncludePaths">Base directories to generate include paths relative to</param>
		/// <param name="PrivateIncludePaths">Base directories to generate include paths relative to, if the output file is also in this directory</param>
		/// <param name="SystemIncludePaths">Base directories to generate system include paths relative to</param>
		/// <param name="OutputFiles">The files to output</param>
		/// <param name="bWithP4">If true, read-only files will be checked out of Perforce</param>
		/// <param name="Log">Log writer</param>
		/// <returns>True if the files were written successfully</returns>
		static bool WriteOptimizedFiles(DirectoryReference InputDir, DirectoryReference OutputDir, HashList<DirectoryReference> PublicIncludePaths, Dictionary<BuildModule, HashList<DirectoryReference>> PrivateIncludePaths, HashList<DirectoryReference> SystemIncludePaths, IEnumerable<SourceFile> OutputFiles, bool bWithP4, bool bWithVerboseLog, bool bRemoveForwardDeclarations, LineBasedTextWriter Log)
		{
			// Find all the output files that need to be written
			Dictionary<FileReference, string> OutputFileContents = new Dictionary<FileReference, string>();
			foreach(SourceFile OutputFile in OutputFiles)
			{
				HashList<DirectoryReference> IncludePaths;
				if((OutputFile.Flags & SourceFileFlags.Public) != 0 || OutputFile.Module == null || !PrivateIncludePaths.TryGetValue(OutputFile.Module, out IncludePaths))
				{
					IncludePaths = PublicIncludePaths;
				}

				StringWriter Writer = new StringWriter();
				OutputFile.Write(IncludePaths.OrderByDescending(x => x.FullName.Length), SystemIncludePaths, Writer, bRemoveForwardDeclarations, Log);

				string Contents = Writer.ToString();

				FileReference NewLocation = FileReference.Combine(OutputDir, OutputFile.Location.MakeRelativeTo(InputDir));
				if(!FileReference.Exists(NewLocation) || File.ReadAllText(NewLocation.FullName).TrimEnd() != Contents.TrimEnd())
				{
					OutputFileContents.Add(NewLocation, Contents);
				}
				if(bWithVerboseLog)
				{
					File.WriteAllLines(NewLocation.FullName + ".txt", OutputFile.VerboseOutput);
				}
			}
			return WriteOutputFiles(OutputDir, OutputFileContents, bWithP4, Log);
		}

		/// <summary>
		/// Write all the output files, optionally checking them out of Perforce
		/// </summary>
		/// <param name="OutputDir">The directory to write all the output files to</param>
		/// <param name="OutputFileContents">The files to write</param>
		/// <param name="bWithP4">If true, read-only files will be checked out of Perforce</param>
		/// <param name="Log">Log writer</param>
		/// <returns>True if the files were written successfully</returns>
		static bool WriteOutputFiles(DirectoryReference OutputDir, Dictionary<FileReference, string> OutputFileContents, bool bWithP4, LineBasedTextWriter Log)
		{
			// Create the output directory
			DirectoryReference.CreateDirectory(OutputDir);

			// Check out any that are read-only from perforce
			if(bWithP4)
			{
				List<FileReference> ReadOnlyFiles = OutputFileContents.Keys.Where(x => File.Exists(x.FullName) && (File.GetAttributes(x.FullName) & FileAttributes.ReadOnly) != 0).ToList();
				if(ReadOnlyFiles.Count > 0)
				{
					// Figure out where Perfoce is
					FileReference PerforceLocation;
					if(!Utility.TryFindProgramInPath("p4.exe", out PerforceLocation))
					{
						Log.WriteLine("Couldn't find location of p4.exe");
						return false;
					}

					// Write out a list of files to check out. This is much faster than checking them out individually.
					Log.WriteLine("Opening output files for edit...");
					FileReference EditList = FileReference.Combine(OutputDir, "p4-edit.txt");
					File.WriteAllLines(EditList.FullName, ReadOnlyFiles.Select(x => x.FullName));

					// Run P4
					if(Utility.Run(PerforceLocation, String.Format("-x \"{0}\" edit", EditList.FullName), OutputDir, Log) != 0)
					{
						Log.WriteLine("Failed to check out files from P4.");
						return false;
					}
				}
			}

			// Write out all the modified files
			Log.WriteLine("Writing output files...");
			foreach(KeyValuePair<FileReference, string> OutputFile in OutputFileContents)
			{
				Log.WriteLine("{0}", OutputFile.Key);
				DirectoryReference.CreateDirectory(OutputFile.Key.Directory);
				File.WriteAllText(OutputFile.Key.FullName, OutputFile.Value);
			}
			return true;
		}
	}
}
