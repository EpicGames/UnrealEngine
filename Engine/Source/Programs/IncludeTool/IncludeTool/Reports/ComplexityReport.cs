// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Reports
{
	/// <summary>
	/// Class containing information about a source file's complexity
	/// </summary>
	class ComplexityData
	{
		/// <summary>
		/// The number of preprocessed lines in this source file, were it to be preprocessed in isolation
		/// </summary>
		public int NumPreprocessedLines;

		/// <summary>
		/// Set of files which directly or indirectly include this file
		/// </summary>
		public HashSet<SourceFile> IncludedBy = new HashSet<SourceFile>();

		/// <summary>
		/// Set of files which directly include this file
		/// </summary>
		public HashSet<SourceFile> DirectlyIncludedBy = new HashSet<SourceFile>();
	}

	/// <summary>
	/// Utility functions to generate a report detailing the complexity of each preprocessed file
	/// </summary>
	static class ComplexityReport
	{
		/// <summary>
		/// Generate a report showing the number of preprocessed lines in the selected files
		/// </summary>
		/// <param name="Files">The files to include in the report</param>
		/// <param name="ReportFileLocation">Output file for the report</param>
		/// <param name="Log">Writer for log output</param>
		public static void Generate(FileReference ReportFileLocation, IEnumerable<SourceFile> Files, LineBasedTextWriter Log)
		{
			Log.WriteLine("Writing {0}...", ReportFileLocation.FullName);

			// Build a list of all the files in the report
			SourceFile[] AllFiles = Files.SelectMany(x => FindIncludedFiles(x)).Distinct().ToArray();

			// Find a map of file to the number of preprocessed lines in it
			Dictionary<SourceFile, ComplexityData> FileToReportData = new Dictionary<SourceFile, ComplexityData>();
			foreach(SourceFile File in AllFiles)
			{
				// Create the complexity data for this file
				ComplexityData ReportData = FindOrAddComplexityData(File, FileToReportData);
				Debug.Assert(ReportData.NumPreprocessedLines == 0);

				// Calculate the preprocessed line count, and update each file it includes with this file
				foreach(SourceFile IncludedFile in FindIncludedFiles(File))
				{
					ComplexityData IncludedFileReportData = FindOrAddComplexityData(IncludedFile, FileToReportData);
					IncludedFileReportData.IncludedBy.Add(File);
					ReportData.NumPreprocessedLines += IncludedFile.Text.Lines.Length;
				}

				// Add this file to each file it directly includes
				foreach(PreprocessorMarkup Markup in File.Markup)
				{
					if(Markup.Type == PreprocessorMarkupType.Include && Markup.IsActive)
					{
						foreach(SourceFile IncludedFile in Markup.OutputIncludedFiles)
						{
							ComplexityData IncludedFileReportData = FindOrAddComplexityData(IncludedFile, FileToReportData);
							IncludedFileReportData.DirectlyIncludedBy.Add(File);
						}
					}
				}
			}

			// Write out a CSV report containing the list of files and their line counts
			using (StreamWriter Writer = new StreamWriter(ReportFileLocation.FullName))
			{
				Writer.WriteLine("File,Line Count,Num Indirect Includes,Num Direct Includes,Direct Includes");
				foreach(KeyValuePair<SourceFile, ComplexityData> Pair in FileToReportData.OrderByDescending(x => x.Value.NumPreprocessedLines))
				{
					string IncludedByList = String.Join(", ", Pair.Value.DirectlyIncludedBy.Select(x => GetDisplayName(x)).OrderBy(x => x));
					Writer.WriteLine("{0},{1},{2},{3},\"{4}\"", GetDisplayName(Pair.Key), Pair.Value.NumPreprocessedLines, Pair.Value.IncludedBy.Count, Pair.Value.DirectlyIncludedBy.Count, IncludedByList);
				}
			}
		}

		/// <summary>
		/// Get the simple display name for a source file
		/// </summary>
		/// <param name="File">The source file to get a name for</param>
		/// <returns>The normalized path or full path for the given file</returns>
		static string GetDisplayName(SourceFile File)
		{
			WorkspaceFile WorkspaceFile = Workspace.GetFile(File.Location);
			return WorkspaceFile.NormalizedPathFromBranchRoot ?? File.Location.FullName;
		}

		/// <summary>
		/// Gets a ComplexityData object for the given source file, or creates it if it does not exist.
		/// </summary>
		/// <param name="SourceFile">The source file to check for</param>
		/// <param name="FileToComplexityData">Mapping of source file to complexity data</param>
		/// <returns>The complexity data object for the given file</returns>
		static ComplexityData FindOrAddComplexityData(SourceFile SourceFile, Dictionary<SourceFile, ComplexityData> FileToComplexityData)
		{
			ComplexityData ReportData;
			if(!FileToComplexityData.TryGetValue(SourceFile, out ReportData))
			{
				// Add the new data
				ReportData = new ComplexityData();
				FileToComplexityData.Add(SourceFile, ReportData);
			}
			return ReportData;
		}
		
		/// <summary>
		/// Recurse through this file and all the files it includes.
		/// </summary>
		/// <returns>All the files included by this file</returns>
		static List<SourceFile> FindIncludedFiles(SourceFile File)
		{
			List<SourceFile> IncludedFiles = new List<SourceFile>();
			HashSet<SourceFile> UniqueIncludedFiles = new HashSet<SourceFile>();
			FindIncludedFilesInternal(File, IncludedFiles, UniqueIncludedFiles);
			return IncludedFiles;
		}

		/// <summary>
		/// Recurse through this file and all the files it includes, invoking a delegate for each file on entering and leaving each file.
		/// </summary>
		/// <param name="IncludedFiles">List of included files</param>
		/// <param name="UniqueIncludedFiles">Set of files which have already been visited which have a header guard</param>
		static void FindIncludedFilesInternal(SourceFile File, List<SourceFile> IncludedFiles, HashSet<SourceFile> UniqueIncludedFiles)
		{
			// Check if this file can be included at this point. Assume that all external files have header guards, because they're sometimes
			// in forms that we don't handle.
			if((!File.HasHeaderGuard && (File.Flags & SourceFileFlags.External) == 0) || UniqueIncludedFiles.Add(File))
			{
				// Enter this file
				IncludedFiles.Add(File);

				// Scan all the includes
				for(int MarkupIdx = 0; MarkupIdx < File.Markup.Length; MarkupIdx++)
				{
					PreprocessorMarkup ThisMarkup = File.Markup[MarkupIdx];
					if(ThisMarkup.Type == PreprocessorMarkupType.Include && ThisMarkup.IsActive)
					{
						foreach(SourceFile IncludedFile in ThisMarkup.OutputIncludedFiles)
						{
							FindIncludedFilesInternal(IncludedFile, IncludedFiles, UniqueIncludedFiles);
						}
					}
				}
			}
		}
	}
}
