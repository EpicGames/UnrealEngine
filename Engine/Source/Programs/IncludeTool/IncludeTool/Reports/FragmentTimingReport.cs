// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Reports
{
	/// <summary>
	/// Timing data for a single compile
	/// </summary>
	struct FragmentTimingSample
	{
		/// <summary>
		/// The total compile time
		/// </summary>
		public double TotalTime;

		/// <summary>
		/// The total frontend compile time
		/// </summary>
		public double FrontendTime;

		/// <summary>
		/// The total backend compile time
		/// </summary>
		public double BackendTime;
	}

	/// <summary>
	/// Stores timing data for compiling a fragment
	/// </summary>
	class FragmentTimingData
	{
		/// <summary>
		/// The unique name for this path
		/// </summary>
		public string UniqueName;

		/// <summary>
		/// The unique digest for this sequence
		/// </summary>
		public string Digest;

		/// <summary>
		/// Data for compiling the previous fragment
		/// </summary>
		public FragmentTimingData PrevFragmentData;

		/// <summary>
		/// The current fragment
		/// </summary>
		public SourceFragment Fragment;

		/// <summary>
		/// Path to the intermediate file
		/// </summary>
		public FileReference IntermediateFile;

		/// <summary>
		/// The compile environment for this fragment
		/// </summary>
		public CompileEnvironment CompileEnvironment;

		/// <summary>
		/// List of samples
		/// </summary>
		public List<FragmentTimingSample> Samples = new List<FragmentTimingSample>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="PrevFragmentData">Timing data for the previous fragment</param>
		/// <param name="Fragment">The fragment to compile</param>
		/// <param name="CompilerExe">Path to the compiler</param>
		/// <param name="ResponseFile">Path to the compiler response file</param>
		/// <param name="LogFile">Path to the log file</param>
		public FragmentTimingData(string UniqueName, string Digest, FragmentTimingData PrevFragmentData, SourceFragment[] Fragments, List<Tuple<int, SourceFile>> IncludeHistory, FileReference IntermediateFile, CompileEnvironment CompileEnvironment)
		{
			this.UniqueName = UniqueName;
			this.Digest = Digest;
			this.PrevFragmentData = PrevFragmentData;
			this.Fragment = Fragments[Fragments.Length - 1];
			this.IntermediateFile = IntermediateFile;
			this.CompileEnvironment = CompileEnvironment;

			// Write the source file
			using (StreamWriter Writer = new StreamWriter(IntermediateFile.FullName))
			{
				int IncludeHistoryIdx = 0;
				List<SourceFile> IncludeStack = new List<SourceFile>();
				for(int Idx = 0; Idx < Fragments.Length; Idx++)
				{
					SourceFragment Fragment = Fragments[Idx];

					// Figure out which tag it's bound to
					int Tag = (Fragment.File.Counterpart == null)? Idx : -1;

					// Update the stack for new includes
					while(IncludeHistoryIdx < IncludeHistory.Count && IncludeHistory[IncludeHistoryIdx].Item1 == Idx)
					{
						if(IncludeHistory[IncludeHistoryIdx].Item2 == null)
						{
							SourceFile IncludeFile = IncludeStack[IncludeStack.Count - 1];
							IncludeStack.RemoveAt(IncludeStack.Count - 1);
							Writer.WriteLine("{0}// END INCLUDE {1}", new string(' ', IncludeStack.Count * 4), IncludeFile.Location.FullName);
							IncludeHistoryIdx++;
						}
						else if(IncludeHistoryIdx + 1 < IncludeHistory.Count && IncludeHistory[IncludeHistoryIdx + 1].Item2 == null)
						{
							IncludeHistoryIdx += 2;
						}
						else
						{
							SourceFile IncludeFile = IncludeHistory[IncludeHistoryIdx].Item2;
							Writer.WriteLine("{0}// INCLUDE {1}", new string(' ', IncludeStack.Count * 4), IncludeFile.Location.FullName);
							IncludeStack.Add(IncludeFile);
							IncludeHistoryIdx++;
						}
					}

					// Get the text for this node
					Writer.WriteLine("{0}#include \"{1}\"", new string(' ', (IncludeStack.Count - 0) * 4), Fragment.Location.FullName);
				}
			}
		}

		/// <summary>
		/// Compiles the fragment and adds the compile time to the list
		/// </summary>
		public void Compile(DirectoryReference IntermediateDir, int SampleIdx)
		{
			FileReference LogFile = FileReference.Combine(IntermediateDir, String.Format("{0}.sample{1}.txt", UniqueName, SampleIdx + 1));
			using (StreamWriter LogWriter = new StreamWriter(LogFile.FullName, true))
			{
				FileReference ResponseFile = FileReference.Combine(IntermediateDir, String.Format("{0}.sample{1}.response", UniqueName, SampleIdx + 1));

				// Write the response file
				CompileEnvironment WorkerCompileEnvironment = new CompileEnvironment(CompileEnvironment);
				if(WorkerCompileEnvironment.CompilerType == CompilerType.Clang)
				{
					WorkerCompileEnvironment.Options.Add(new CompileOption("-o", FileReference.Combine(IntermediateDir, UniqueName + ".o").FullName));
				}
				else if(WorkerCompileEnvironment.CompilerType == CompilerType.VisualC)
				{
					WorkerCompileEnvironment.Options.RemoveAll(x => x.Name == "/Z7" || x.Name == "/Zi" || x.Name == "/ZI");
					WorkerCompileEnvironment.Options.Add(new CompileOption("/Fo", IntermediateFile.FullName + ".obj"));
					WorkerCompileEnvironment.Options.Add(new CompileOption("/WX", null));
					WorkerCompileEnvironment.Options.Add(new CompileOption("/Bt", null));
				}
				else
				{
					throw new NotImplementedException();
				}
				WorkerCompileEnvironment.WriteResponseFile(ResponseFile, IntermediateFile);

				// Spawn the compiler
				using (Process NewProcess = new Process())
				{
					List<string> LogLines = new List<string>();
					DataReceivedEventHandler OutputHandler = (x, y) => { if (!String.IsNullOrEmpty(y.Data)) { LogWriter.WriteLine(y.Data); LogLines.Add(y.Data); } };

					NewProcess.StartInfo.FileName = CompileEnvironment.Compiler.FullName;
					NewProcess.StartInfo.Arguments = String.Format("\"@{0}\"", ResponseFile);
					NewProcess.StartInfo.UseShellExecute = false;
					NewProcess.StartInfo.RedirectStandardOutput = true;
					NewProcess.StartInfo.RedirectStandardError = true;
					NewProcess.OutputDataReceived += OutputHandler;
					NewProcess.ErrorDataReceived += OutputHandler;

					Stopwatch Timer = Stopwatch.StartNew();
					NewProcess.Start();
					NewProcess.BeginOutputReadLine();
					NewProcess.BeginErrorReadLine();

					if (NewProcess.WaitForExit(10 * 60 * 1000))
					{
						// WaitForExit with a timeout does not wait for output data to be flushed, so issue a normal WaitForExit call here too
						NewProcess.WaitForExit();
						Timer.Stop();

						FragmentTimingSample Sample = new FragmentTimingSample();
						Sample.TotalTime = Timer.Elapsed.TotalSeconds;
						if(WorkerCompileEnvironment.CompilerType == CompilerType.VisualC)
						{
							foreach(string LogLine in LogLines)
							{
								if(TryParseCompileTime(LogLine, "c1xx.dll", out Sample.FrontendTime))
								{
									break;
								}
							}
							foreach(string LogLine in LogLines)
							{
								if(TryParseCompileTime(LogLine, "c2.dll", out Sample.BackendTime))
								{
									break;
								}
							}
						}
						Samples.Add(Sample);

						LogWriter.WriteLine("Compile finished in {0}s", Timer.Elapsed.TotalSeconds);
					}
					else
					{
						NewProcess.Kill();
						NewProcess.WaitForExit();
						Timer.Stop();
						Samples.Add(new FragmentTimingSample());
						LogWriter.WriteLine("Timeout; terminating process after {0}s", Timer.Elapsed.TotalSeconds);
					}

					LogWriter.WriteLine();
				}
			}
		}

		static bool TryParseCompileTime(string Line, string ModuleName, out double Time)
		{
			const string Prefix = "time(";
			if(!Line.StartsWith(Prefix) || !Line.EndsWith("s"))
			{
				Time = 0.0;
				return false;
			}

			int EndIdx = Line.IndexOf(")=");
			if(EndIdx == -1 || String.Compare(Path.GetFileName(Line.Substring(Prefix.Length, EndIdx - Prefix.Length)), ModuleName, true) != 0)
			{
				Time = 0.0;
				return false;
			}

			return Double.TryParse(Line.Substring(EndIdx + 2, (Line.Length - 1) - (EndIdx + 2)), out Time);
		}
	}

	/// <summary>
	/// Generates timing data for compiling each fragment
	/// </summary>
	static class FragmentTimingReport
	{
		/// <summary>
		/// Gather compile time telemetry for the given files
		/// </summary>
		/// <param name="FileToCompileEnvironment">Mapping of source file to the environment used to compile it</param>
		/// <param name="WorkingDir">The working directory for output files</param>
		/// <param name="NumSamples">Number of samples to take</param>
		/// <param name="MaxParallel">Maximum number of tasks to run in parallel.</param>
		/// <param name="Log">Log writer</param>
		public static void Generate(DirectoryReference InputDir, DirectoryReference WorkingDir, Dictionary<SourceFile, CompileEnvironment> FileToCompileEnvironment, int NumSamples, int Shard, int NumShards, int MaxParallel, LineBasedTextWriter Log)
		{
			Stopwatch Timer = Stopwatch.StartNew();

			// Create an intermediate directory
			DirectoryReference IntermediateDir = DirectoryReference.Combine(WorkingDir, "Timing");
			DirectoryReference.CreateDirectory(IntermediateDir);

			// Map of unique fragment to timing data
			Dictionary<SourceFragment, FragmentTimingData> FragmentToTimingData = new Dictionary<SourceFragment, FragmentTimingData>();

			// Map of unique fragment key to timing data
			Dictionary<string, FragmentTimingData> DigestToTimingData = new Dictionary<string, FragmentTimingData>();

			// List of all the sequences to time
			HashSet<string> UniqueNames = new HashSet<string>();
			foreach(KeyValuePair<SourceFile, CompileEnvironment> Pair in FileToCompileEnvironment)
			{
				// Find all the fragments in this file
				List<SourceFragment> Fragments = new List<SourceFragment>();
				List<Tuple<int, SourceFile>> IncludeHistory = new List<Tuple<int, SourceFile>>();
				Pair.Key.FindIncludedFragments(Fragments, IncludeHistory, new HashSet<SourceFile>());

				// Create a sequence for each unique fragment
				FragmentTimingData PrevTimingData = null;
				for(int Idx = 0; Idx < Fragments.Count; Idx++)
				{
					FragmentTimingData TimingData = null;
					if(!FragmentToTimingData.ContainsKey(Fragments[Idx]) || (Idx + 1 < Fragments.Count && !FragmentToTimingData.ContainsKey(Fragments[Idx + 1])))
					{
						// Create a sequence for this fragment
						SourceFragment LastFragment = Fragments[Idx];

						// Create a unique key for this sequence by concatenating all the fragment names
						string Digest = Utility.ComputeDigest(String.Join("\n", Fragments.Take(Idx + 1).Select(x => x.Location.FullName)));

						// Try to get an existing sequence for this key, otherwise create a new one;
						if(!DigestToTimingData.TryGetValue(Digest, out TimingData))
						{
							// Find a unique name for this sequence
							string UniqueName = LastFragment.Location.GetFileName();
							for(int NameIdx = 2; !UniqueNames.Add(UniqueName); NameIdx++)
							{
								UniqueName = String.Format("{0}_{1}{2}", LastFragment.Location.GetFileNameWithoutExtension(), NameIdx, LastFragment.Location.GetExtension());
							}

							// Add the object for this sequence
							FileReference IntermediateFile = FileReference.Combine(IntermediateDir, UniqueName);
							TimingData = new FragmentTimingData(UniqueName, Digest, PrevTimingData, Fragments.Take(Idx + 1).ToArray(), IncludeHistory, IntermediateFile, Pair.Value);
							DigestToTimingData.Add(Digest, TimingData);
						}

						// Add it to the unique mapping of fragments
						if(!FragmentToTimingData.ContainsKey(LastFragment))
						{
							FragmentToTimingData[LastFragment] = TimingData;
						}
					}
					PrevTimingData = TimingData;
				}
			}

			// Read any existing shard timing data in the output folder
			foreach(FileReference IntermediateFile in DirectoryReference.EnumerateFiles(IntermediateDir, "*.csv"))
			{
				string[] Lines = File.ReadAllLines(IntermediateFile.FullName);
				foreach(string Line in Lines.Skip(1))
				{
					string[] Tokens = Line.Split(',');
					if(Tokens.Length == 5)
					{
						FragmentTimingData TimingData;
						if(DigestToTimingData.TryGetValue(Tokens[1], out TimingData) && TimingData.Samples.Count < NumSamples)
						{
							FragmentTimingSample Sample = new FragmentTimingSample();
							Sample.TotalTime = Double.Parse(Tokens[2]);
							Sample.FrontendTime = Double.Parse(Tokens[3]);
							Sample.BackendTime = Double.Parse(Tokens[4]);
							TimingData.Samples.Add(Sample);
						}
					}
				}
			}

			// Find all the remaining fragments, and repeat each one by the number of times it has to be executed
			List<FragmentTimingData> FilteredFragments = DigestToTimingData.Values.ToList();
			FilteredFragments.RemoveAll(x => (int)(Math.Abs((long)x.Digest.GetHashCode()) % NumShards) != (Shard - 1));

			// Get the initial number of compile times for each fragment. We avoid saving before this number.
			List<int> InitialCompileCount = FilteredFragments.Select(x => x.Samples.Count).ToList();

			// Create all the actions to execute
			List<Action> Actions = new List<Action>();
			foreach(FragmentTimingData Fragment in FilteredFragments)
			{
				FragmentTimingData FragmentCopy = Fragment;
				for(int SampleIdx = Fragment.Samples.Count; SampleIdx < NumSamples; SampleIdx++)
				{
					int SampleIdxCopy = SampleIdx;
					Actions.Add(() => FragmentCopy.Compile(IntermediateDir, SampleIdxCopy));
				}
			}

			// Randomize the order to ensure that compile times are not consistently affected by other files being compiled simultaneously.
			Random Random = new Random();
			Actions = Actions.OrderBy(x => Random.Next()).ToList();

			// Compile them all
			if(Actions.Count > 0)
			{
				Utility.ParallelForWithStatus("Compiling fragments...", 0, Actions.Count, new ParallelOptions { MaxDegreeOfParallelism = MaxParallel }, Idx => Actions[Idx](), Log);
			}

			// Write out the results
			if(NumShards > 1)
			{
				// If we're running a sharded build, write out intermediate files containing the results
				FileReference OutputFile = FileReference.Combine(IntermediateDir, String.Format("Shard{0}.csv", Shard));
				using (StreamWriter Writer = new StreamWriter(OutputFile.FullName))
				{
					Writer.WriteLine("Name,Digest,TotalTime,FrontendTime,BackendTime");
					for(int Idx = 0; Idx < FilteredFragments.Count; Idx++)
					{
						FragmentTimingData FilteredFragment = FilteredFragments[Idx];
						for(int SampleIdx = InitialCompileCount[Idx]; SampleIdx < FilteredFragment.Samples.Count; SampleIdx++)
						{
							FragmentTimingSample Sample = FilteredFragment.Samples[SampleIdx];
							Writer.WriteLine("{0},{1},{2},{3},{4}", FilteredFragment.UniqueName, FilteredFragment.Digest, Sample.TotalTime, Sample.FrontendTime, Sample.BackendTime);
						}
					}
				}
			}
			else
			{
				// Write out the fragment report
				FileReference FragmentReport = FileReference.Combine(WorkingDir, "Timing.csv");
				Log.WriteLine("Writing {0}...", FragmentReport);
				using (StreamWriter Writer = new StreamWriter(FragmentReport.FullName))
				{
					// Write the header
					Writer.Write("Fragment,MinLine,MaxLine");

					// Write the labels for each sample type
					string[] Types = new string[] { "Total", "Frontend", "Backend" };
					for(int Idx = 0; Idx < Types.Length; Idx++)
					{
						for(int SampleIdx = 0; SampleIdx < NumSamples; SampleIdx++)
						{
							Writer.Write(",{0}{1}", Types[Idx], SampleIdx + 1);
						}
						Writer.Write(",{0}Min,{0}Max,{0}Avg,{0}Exc", Types[Idx]);
					}
					Writer.WriteLine();

					// Write all the results
					Func<FragmentTimingSample, double>[] TimeFieldDelegates = new Func<FragmentTimingSample, double>[]{ x => x.TotalTime, x => x.FrontendTime, x => x.BackendTime };
					foreach(FragmentTimingData TimingData in FragmentToTimingData.Values)
					{
						Writer.Write("{0},{1},{2}", TimingData.Fragment.Location.GetFileName(), TimingData.Fragment.MarkupMin + 1, TimingData.Fragment.MarkupMax + 1);
						foreach(Func<FragmentTimingSample, double> TimeFieldDelegate in TimeFieldDelegates)
						{
							foreach(FragmentTimingSample Sample in TimingData.Samples)
							{
								Writer.Write(",{0:0.000}", TimeFieldDelegate(Sample));
							}

							Writer.Write(",{0:0.000}", TimingData.Samples.Min(x => TimeFieldDelegate(x)));
							Writer.Write(",{0:0.000}", TimingData.Samples.Max(x => TimeFieldDelegate(x)));
							Writer.Write(",{0:0.000}", TimingData.Samples.Average(x => TimeFieldDelegate(x)));

							if(TimingData.PrevFragmentData == null)
							{
								Writer.Write(",{0:0.000}", TimingData.Samples.Average(x => TimeFieldDelegate(x)));
							}
							else
							{
								Writer.Write(",{0:0.000}", TimingData.Samples.Average(x => TimeFieldDelegate(x)) - TimingData.PrevFragmentData.Samples.Average(x => TimeFieldDelegate(x)));
							}
						}
						Writer.WriteLine();
					}
				}
			}
		}
	}
}
