// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Reports
{
	class PchInfo
	{
		public FileReference PchFile;
		public HashList<BuildModule> PublicIncludePathModules = new HashList<BuildModule>();
		public HashSet<FileReference> SourceFiles = new HashSet<FileReference>();
		public Dictionary<SourceFile, int> IncludedFiles = new Dictionary<SourceFile, int>();

		public PchInfo(FileReference PchFile)
		{
			this.PchFile = PchFile;
		}
	}

	/// <summary>
	/// Generates a report listing the most frequently referenced headers in the codebase
	/// </summary>
	static class PchReport
	{
		/// <summary>
		/// Generate a report showing the number of preprocessed lines in the selected files
		/// </summary>
		/// <param name="ReportFileLocation">Output file for the report</param>
		/// <param name="InputDir"></param>
		/// <param name="Target"></param>
		/// <param name="SourceFileToCompileEnvironment"></param>
		/// <param name="Log">Writer for log output</param>
		public static void Generate(FileReference ReportFileLocation, DirectoryReference InputDir, BuildTarget Target, Dictionary<SourceFile, CompileEnvironment> SourceFileToCompileEnvironment, LineBasedTextWriter Log)
		{
			Log.WriteLine("Writing {0}...", ReportFileLocation.FullName);

			// Create a map from source file to the number of times it's included
			Dictionary<FileReference, PchInfo> FileToPchInfo = new Dictionary<FileReference, PchInfo>();
			FindPchInfo(Target, SourceFileToCompileEnvironment, FileToPchInfo, Log);

			// Write out a CSV report containing the list of files and their line counts
			using (StreamWriter Writer = new StreamWriter(ReportFileLocation.FullName))
			{
				Writer.WriteLine("PCH,File,Num Includes,Pct Includes");
				foreach(FileReference PchFile in FileToPchInfo.Keys)
				{
					PchInfo PchInfo = FileToPchInfo[PchFile];
					foreach(KeyValuePair<SourceFile, int> Pair in PchInfo.IncludedFiles.OrderByDescending(x => x.Value))
					{
						if((Pair.Key.Flags & SourceFileFlags.Pinned) == 0 && (Pair.Key.Flags & SourceFileFlags.External) == 0 && (Pair.Key.Flags & SourceFileFlags.Inline) == 0)
						{
							Writer.WriteLine("{0},{1},{2},{3:0.00}", PchFile.GetFileName(), Pair.Key.Location.MakeRelativeTo(InputDir), Pair.Value, (Pair.Value * 100.0) / PchInfo.SourceFiles.Count);
						}
					}
				}
			}
		}

		/// <summary>
		/// Generate optimized PCHs which include headers used by a ratio of the source files
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <param name="SourceFileToCompileEnvironment">Source files to consider</param>
		/// <param name="IncludePaths">Include paths to base output includes from</param>
		/// <param name="SystemIncludePaths">System include paths to base output includes from</param>
		/// <param name="OutputFileContents">Dictionary which receives the files to output</param>
		/// <param name="MinRatio">Ratio of source files which must include a header for it to be included in the pch</param>
		/// <param name="Log">Writer for log output</param>
		public static void GeneratePCHs(BuildTarget Target, Dictionary<SourceFile, CompileEnvironment> SourceFileToCompileEnvironment, IEnumerable<DirectoryReference> IncludePaths, IEnumerable<DirectoryReference> SystemIncludePaths, Dictionary<FileReference, string> OutputFileContents, float MinRatio, LineBasedTextWriter Log)
		{
			Log.WriteLine("Optimizing precompiled headers...");

			// Create a map from source file to the number of times it's included
			Dictionary<FileReference, PchInfo> FileToPchInfo = new Dictionary<FileReference, PchInfo>();
			FindPchInfo(Target, SourceFileToCompileEnvironment, FileToPchInfo, Log);

			// Create an ordering of all the modules
			Dictionary<BuildModule, int> ModuleToIndex = new Dictionary<BuildModule, int>();
			FindModuleOrder(SourceFileToCompileEnvironment.Keys.Select(x => x.Module).Distinct(), ModuleToIndex, new HashSet<BuildModule>());

			// Create the output files
			foreach(FileReference PchFile in FileToPchInfo.Keys)
			{
				PchInfo PchInfo = FileToPchInfo[PchFile];

				// Get the minimum number of includes to use in the PCH
				int MinIncludes = (int)(MinRatio * PchInfo.SourceFiles.Count);

				// Get a list of all the files to include above a threshold
				List<SourceFile> IncludeFiles = new List<SourceFile>();
				foreach(SourceFile IncludedFile in PchInfo.IncludedFiles.Where(x => x.Value >= MinIncludes).OrderByDescending(x => x.Value).Select(x => x.Key))
				{
					if((IncludedFile.Flags & SourceFileFlags.Pinned) == 0 && (IncludedFile.Flags & SourceFileFlags.External) == 0 && (IncludedFile.Flags & SourceFileFlags.Inline) == 0 && PchInfo.PublicIncludePathModules.Contains(IncludedFile.Module))
					{
						IncludeFiles.Add(IncludedFile);
					}
				}

				// Generate the output file text
				StringBuilder Contents = new StringBuilder();
				using (StringWriter Writer = new StringWriter(Contents))
				{
					Writer.WriteLine("// Copyright 1998-{0} Epic Games, Inc. All Rights Reserved.", DateTime.Now.Year);
					Writer.WriteLine();
					Writer.WriteLine("#pragma once");
					foreach(IGrouping<BuildModule, SourceFile> Group in IncludeFiles.GroupBy(x => x.Module).OrderBy(x => ModuleToIndex[x.Key]))
					{
						Writer.WriteLine();
						Writer.WriteLine("// From {0}:", Group.Key.Name);
						foreach(SourceFile IncludeFile in Group)
						{
							string Include;
							if(SourceFile.TryFormatInclude(PchFile.Directory, IncludeFile.Location, IncludePaths, SystemIncludePaths, out Include))
							{
								Writer.WriteLine("#include {0}", Include);
							}
						}
					}
				}

				// Add it to the output map
				OutputFileContents.Add(PchFile, Contents.ToString());
			}
		}

		/// <summary>
		/// Find a mapping from PCH to the most included files by the files using it
		/// </summary>
		/// <param name="SourceFileToCompileEnvironment">Files being compiled</param>
		/// <param name="PchToIncludeFileCount">Mapping of PCH to included files</param>
		/// <param name="Log">Writer for log messages</param>
		static void FindPchInfo(BuildTarget Target, Dictionary<SourceFile, CompileEnvironment> SourceFileToCompileEnvironment, Dictionary<FileReference, PchInfo> FileToPchInfo, LineBasedTextWriter Log)
		{
			// Create a map of module to the shared PCH it uses
			Dictionary<BuildModule, FileReference> ModuleToPch = new Dictionary<BuildModule, FileReference>();

			// Recurse through all the includes for each source file
			Dictionary<FileReference, int> UsingPchCount = new Dictionary<FileReference, int>();
			foreach (KeyValuePair<SourceFile, CompileEnvironment> Pair in SourceFileToCompileEnvironment)
			{
				// Figure out which module it's in
				BuildModule Module = Pair.Key.Module;

				// Determine which PCH it's using
				FileReference UsingPch;
				if (!ModuleToPch.TryGetValue(Module, out UsingPch))
				{
					if (Module.PrivatePCH != null)
					{
						UsingPch = Module.PrivatePCH;
					}
					else if (Module.PCHUsage == BuildModulePCHUsage.UseExplicitOrSharedPCHs || Module.PCHUsage == BuildModulePCHUsage.UseSharedPCHs || Module.PCHUsage == BuildModulePCHUsage.Default)
					{
						HashSet<BuildModule> PossibleModules = new HashSet<BuildModule>(Module.NonCircularDependencies.Where(x => x.SharedPCH != null));
						foreach (BuildModule PossibleModule in PossibleModules.ToArray())
						{
							PossibleModules.ExceptWith(PossibleModule.NonCircularDependencies);
						}
						if(PossibleModules.Count == 0)
						{
							Log.WriteLine("warning: No valid PCH found for {0}", Module);
						}
						else if (PossibleModules.Count == 1)
						{
							UsingPch = PossibleModules.First().SharedPCH;
						}
						else
						{
							Log.WriteLine("warning: Multiple valid PCHs for {0}: {1}", Module, String.Join(",", PossibleModules.Select(x => x.Name)));
						}
					}
					else
					{
						Log.WriteLine("warning: Unknown PCH for {0}", Module);
					}
					ModuleToPch[Module] = UsingPch;
				}

				// Make sure we're using a PCH
				if(UsingPch != null)
				{
					// Get the info for this PCH
					PchInfo Info;
					if(!FileToPchInfo.TryGetValue(UsingPch, out Info))
					{
						Info = new PchInfo(UsingPch);

						Info.PublicIncludePathModules.Add(Target.Modules.First(x => UsingPch.IsUnderDirectory(x.Directory)));
						for(int Idx = 0; Idx < Info.PublicIncludePathModules.Count; Idx++)
						{
							BuildModule NextModule = Info.PublicIncludePathModules[Idx];
							Info.PublicIncludePathModules.UnionWith(NextModule.PublicDependencyModules.Except(NextModule.CircularlyReferencedModules));
						}

						FileToPchInfo.Add(UsingPch, Info);
					}

					// Increment the number of files using this PCH
					Info.SourceFiles.Add(Pair.Key.Location);

					// Find all the included files
					HashSet<SourceFile> IncludedFiles = new HashSet<SourceFile>();
					FindIncludedFiles(Pair.Key, IncludedFiles);

					// Update the counts for each one
					foreach (SourceFile IncludedFile in IncludedFiles)
					{
						int IncludeCount;
						Info.IncludedFiles.TryGetValue(IncludedFile, out IncludeCount);
						Info.IncludedFiles[IncludedFile] = IncludeCount + 1;
					}
				}
			}
		}

		/// <summary>
		/// Find a module ordering for output PCH files
		/// </summary>
		/// <param name="Modules">Sequence of modules to order</param>
		/// <param name="ModuleToIndex">Dictionary which receives an index for each module</param>
		/// <param name="VisitedModules">Set of visited modules, to prevent circular dependencies</param>
		static void FindModuleOrder(IEnumerable<BuildModule> Modules, Dictionary<BuildModule, int> ModuleToIndex, HashSet<BuildModule> VisitedModules)
		{
			foreach(BuildModule Module in Modules)
			{
				if(!ModuleToIndex.ContainsKey(Module) && VisitedModules.Add(Module))
				{
					HashSet<BuildModule> ReferencedModules = new HashSet<BuildModule>();
					ReferencedModules.UnionWith(Module.PublicDependencyModules);
					ReferencedModules.UnionWith(Module.PrivateDependencyModules);
					ReferencedModules.ExceptWith(Module.CircularlyReferencedModules);
					FindModuleOrder(ReferencedModules, ModuleToIndex, VisitedModules);

					ModuleToIndex.Add(Module, ModuleToIndex.Count);
				}
			}
		}

		/// <summary>
		/// Recurse through this file and all the files it includes, invoking a delegate for each file on entering and leaving each file.
		/// </summary>
		/// <param name="IncludedFiles">List of included files</param>
		/// <param name="UniqueIncludedFiles">Set of files which have already been visited which have a header guard</param>
		static void FindIncludedFiles(SourceFile File, HashSet<SourceFile> IncludedFiles)
		{
			foreach(PreprocessorMarkup Markup in File.Markup)
			{
				if(Markup.Type == PreprocessorMarkupType.Include && Markup.IsActive)
				{
					foreach(SourceFile IncludedFile in Markup.OutputIncludedFiles)
					{
						if(IncludedFiles.Add(IncludedFile))
						{
							FindIncludedFiles(IncludedFile, IncludedFiles);
						}
					}
				}
			}
		}
	}
}
