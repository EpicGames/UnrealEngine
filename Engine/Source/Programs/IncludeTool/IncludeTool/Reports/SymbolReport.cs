// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using IncludeTool.Support;
using EpicGames.Core;
using System.IO;

namespace IncludeTool
{
	/// <summary>
	/// Generates a report listing the most frequently referenced symbols in each module
	/// </summary>
	static class SymbolReport
	{
		/// <summary>
		/// Generates a report listing the most frequently referenced opaque symbols in each module
		/// </summary>
		/// <param name="Files">The files to include in the report</param>
		/// <param name="ReportFileLocation">Output file for the report</param>
		/// <param name="Log">Writer for log output</param>
		public static void Generate(FileReference ReportFileLocation, DirectoryReference InputDir, HashSet<SourceFile> PreprocessedFiles, LineBasedTextWriter Log)
		{
			Log.WriteLine("Writing {0}...", ReportFileLocation.FullName);

			// Get a count of files referencing each symbol
			Dictionary<Symbol, int> SymbolToRefCount = new Dictionary<Symbol, int>();
			Dictionary<Symbol, int> SymbolToOpaqueRefCount = new Dictionary<Symbol, int>();
			foreach (SourceFile PreprocessedFile in PreprocessedFiles)
			{
				if (PreprocessedFile.Fragments != null)
				{
					HashSet<Symbol> Symbols = new HashSet<Symbol>();
					HashSet<Symbol> NonOpaqueSymbols = new HashSet<Symbol>();
					foreach(SourceFragment Fragment in PreprocessedFile.Fragments)
					{
						foreach(KeyValuePair<Symbol, SymbolReferenceType> Pair in Fragment.ReferencedSymbols)
						{
							Symbols.Add(Pair.Key);
							if (Pair.Value != SymbolReferenceType.Opaque)
							{
								NonOpaqueSymbols.Add(Pair.Key);
							}
						}
					}

					foreach (Symbol Symbol in Symbols)
					{
						int Count;
						SymbolToRefCount.TryGetValue(Symbol, out Count);
						SymbolToRefCount[Symbol] = Count + 1;

						int OpaqueCount;
						SymbolToOpaqueRefCount.TryGetValue(Symbol, out OpaqueCount);
						SymbolToOpaqueRefCount[Symbol] = NonOpaqueSymbols.Contains(Symbol) ? OpaqueCount : OpaqueCount + 1;
					}
				}
			}

			// Build a map of module to symbols
			MultiValueDictionary<BuildModule, Symbol> ModuleToSymbols = new MultiValueDictionary<BuildModule, Symbol>();
			foreach (Symbol Symbol in SymbolToRefCount.Keys)
			{
				SourceFile File = Symbol.Fragment.File;
				if(File.Module != null)
				{
					ModuleToSymbols.Add(File.Module, Symbol);
				}
			}

			// Write out a CSV report containing the list of symbols and number of files referencing them
			using (StreamWriter Writer = new StreamWriter(ReportFileLocation.FullName))
			{
				Writer.WriteLine("Module,Symbol,Fwd,RefCount,OpaqueRefCount");
				foreach(BuildModule Module in ModuleToSymbols.Keys)
				{
					foreach(Symbol Symbol in ModuleToSymbols[Module].OrderByDescending(x => SymbolToOpaqueRefCount[x]))
					{
						Writer.WriteLine("{0},{1},{2},{3},{4}", Module.Name, Symbol.Name, Symbol.ForwardDeclaration, SymbolToRefCount[Symbol], SymbolToOpaqueRefCount[Symbol]);
					}
				}
			}
		}
	}
}
