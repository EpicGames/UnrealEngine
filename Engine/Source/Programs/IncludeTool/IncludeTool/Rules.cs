// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Class which can modify the rules object at runtime
	/// </summary>
	abstract class RulesMutator
	{
		/// <summary>
		/// Allows a platform specific hook to modify the rules class
		/// </summary>
		public abstract void Run();
	}

	/// <summary>
	/// Contains callback functions to disambiguate and provide metadata for files in this branch
	/// </summary>
	static class Rules
	{
		/// <summary>
		/// List of directories to ignore when scanning the branch for source files
		/// </summary>
		static readonly string[] IgnoredSearchDirectories =
		{
			"/engine/extras/notforlicensees/doxygen/",
			"/engine/plugins/editor/pluginbrowser/templates/",
			"/engine/source/runtime/engine/classes/intrinsic/",
			"/engine/source/thirdparty/llvm/",
			"/engine/source/programs/unrealswarm/private/",
			"/engine/plugins/runtime/packethandlers/compressioncomponents/oodle/source/thirdparty/notforlicensees/oodle/213/win/examples/",
			"/engine/plugins/developer/riderlink/",
			"/engine/source/runtime/symslib/syms/",
		};

		/// <summary>
		/// Callback to filter which directories are searched when looking for source code
		/// </summary>
		/// <param name="Directory">The current directory</param>
		/// <returns>True if the directory should be searched, false otherwise</returns>
		public static bool SearchDirectoryForSource(string NormalizedPath)
		{
			if (IgnoredSearchDirectories.Any(x => x == NormalizedPath))
			{
				return false;
			}

			// Filter out standard non-source directories
			if (NormalizedPath.EndsWith("/binaries/") || NormalizedPath.EndsWith("/saved/") || NormalizedPath.EndsWith("/content/"))
			{
				return false;
			}

			// Ignore intermediate folders; the correct one for this target will be scanned explicitly
			if (NormalizedPath.EndsWith("/intermediate/"))
			{
				return false;
			}
			if (NormalizedPath.StartsWith("/engine/extras/thirdpartynotue/"))
			{
				return false;
			}

			return true;
		}
		
		/// <summary>
		/// List of include tokens which are external files, and do not need to be resolved
		/// </summary>
		static readonly string[] ExternalFileIncludePaths =
		{
			"binkplugin.h",
			"egttypes.h",
			"oodle.h",
			"oodle2.h",
			"xg.h",
			"xmem.h",
			"nvtess.h",
			"jemalloc.h",
			"target/include/json2.h",
			"fios2.h",
			"SDL.h",
			"SDL_timer.h",
			"SDL3/",
			"zlib.h",
			"sqlite3.h",
			"emscripten.h",
			"NvTriStrip.h",
			"libwebsockets.h",
			"include/cef_",
			"include/internal/cef_",
			"opus.h",
			"opus_multistream.h",
			"libyuv/",
			"openssl/",
			"vpx/",
			"nvtt/",
			"mach/",
			"webrtc/",
			"np_",
			"Res/paModel",
			"ft2build.h",
			"hb-private.hh",
			"hb-font-private.hh",
			"curl/curl.h",
			"dbgeng.h",
			"SimplygonSDK.h",
			"OVR_Math.h",
			"clang/AST/",
			"clang/Frontend/",
			"clang/Lex/",
			"clang/Tooling/",
			"llvm/Support/",
			"lua.h",
			"luaxlib.h",
			"lauxlib.h",
			"lualib.h",
			"Leap.h",
			"/engine/extras/thirdpartynotue/",

			// CRT
			"unistd.h",
			"stdio.h",
			"unicode/",
			
			// Android
			"SLES/OpenSLES_Android.h",

			// Windows
			"windows.h",
			"WinSock2.h",
			"Windows.h",
			"Windowsx.h",
			"Ws2tcpip.h",
			"wincodec.h",
			"Ole2.h",
			"OleIdl.h",
			"d3d11_x.h",
			"d3dcompiler_x.h",
			"d3d11shader_x.h",
			"Shellapi.h",
			"VSPerf.h",
			"atlbase.h",
			"Dwmapi.h",
			"DbgHelp.h",
			"iphlpapi.h",
			"Iphlpapi.h",
			"IcmpAPI.h",
			"EtwPlus.h",
			
			// Mac
			"AUEffectBase.h",
			"Security/Security.h",

			// Vorbis
			"vorbis_stream_encoder.h",
			"integral_types.h",

			// UHT
			"StructSpecifiers.def",
			"FunctionSpecifiers.def",
			"InterfaceSpecifiers.def",
			"VariableSpecifiers.def",
			"CheckedMetadataSpecifiers.def",

			// Don't exist any more but are never compiled in; work around it by treating as external
			"Android/AndroidGL4OpenGL.h",
			"../AndroidGL4/AndroidGL4OpenGL.h",
		};

		/// <summary>
		/// Determine whether the given include token is to an external file. If so, it doesn't need to be resolved.
		/// </summary>
		/// <param name="IncludeToken">The include token</param>
		/// <returns>True if the include is to an external file</returns>
		public static bool IsExternalInclude(string IncludePath)
		{
			return ExternalFileIncludePaths.Any(x => IncludePath.StartsWith(x));
		}

		/// <summary>
		/// Determine if the given path is to an external file. If so, it doesn't need to be resolved.
		/// </summary>
		/// <param name="NormalizedPath">Path to the file</param>
		/// <returns>True if the path is to an external file</returns>
		public static bool IsExternalHeaderPath(string NormalizedPath)
		{
			if (NormalizedPath.Contains("/source/thirdparty/") && !NormalizedPath.StartsWith("/engine/source/thirdparty/oculus/common/"))
			{
				return true;
			}
			if (NormalizedPath.StartsWith("/engine/plugins/experimental/phya/source/phya/private/phyalib/"))
			{
				return true;
			}
			if (NormalizedPath.StartsWith("/engine/plugins/runtime/leapmotion/thirdparty/"))
			{
				return true;
			}
			if (NormalizedPath.Contains("/rev.runtime/"))
			{
				return true;
			}
			if (NormalizedPath.EndsWith("/recastmesh.cpp") || NormalizedPath.EndsWith("/recastfilter.cpp") || NormalizedPath.EndsWith("/recastcontour.cpp") || NormalizedPath.EndsWith("/framepro.h") || NormalizedPath.EndsWith("/framepro.cpp") || NormalizedPath.EndsWith("/frameproue4.h") || NormalizedPath.EndsWith("/frameproue4.cpp") || NormalizedPath.EndsWith("/sqlite3.h") || NormalizedPath.EndsWith("/sqlite3.inl") || NormalizedPath.EndsWith("/vorbis_stream_encoder.h") || NormalizedPath.EndsWith("/integral_types.h"))
			{
				return true;
			}
			if (NormalizedPath.Contains("/thirdparty/rapidjson/"))
			{
				return true;
			}
			if (NormalizedPath.EndsWith("/lz4.h") || NormalizedPath.EndsWith("/lz4hc.h") || NormalizedPath.EndsWith("/lz4.cpp") || NormalizedPath.EndsWith("/lz4hc.cpp"))
			{
				return true;
			}
			// Ignore Rider's RiderLink code.
			if (NormalizedPath.Contains("/riderlink/source/riderlink/") || NormalizedPath.Contains("/riderlink/source/rd/"))
			{
				return true;
			}
			// remove symslib files
			if (NormalizedPath.Contains("symslib/syms"))
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// Determines whether to ignore parsing exported symbols from a file
		/// </summary>
		/// <returns>True to ignore exported symbols</returns>
		public static bool ShouldIgnoreExports(string NormalizedPath)
		{
			return NormalizedPath.StartsWith("/engine/plugins/notforlicensees/uephysics/");
		}

		/// <summary>
		/// Determines if the given file path is to a C++ source (or header) file
		/// </summary>
		/// <param name="NormalizedPath">Path to the file, from the branch root</param>
		/// <returns>True if the path represents a source file</returns>
		public static bool IsSourceFile(string NormalizedPath)
		{
			return NormalizedPath.EndsWith(".h") || NormalizedPath.EndsWith(".c") || NormalizedPath.EndsWith(".hpp") || NormalizedPath.EndsWith(".cpp") || NormalizedPath.EndsWith(".inl");
		}

		/// <summary>
		/// List of include paths to ignore. Files under paths in the first entry do NOT include any files under paths in the second entry.
		/// </summary>
		static string[,] IgnoreIncludePatterns = new string[,]
		{
			{ "/portal/source/layers/", "/engine/source/editor/unrealed/" },
			{ "/portal/source/layers/", "/engine/source/runtime/online/icmp/" },
			{ "/engine/source/programs/unreallightmass/", "/engine/source/runtime/engine/" },

		};

		/// <summary>
		/// Checks to see if the given token is a known include macro
		/// </summary>
		/// <param name="TokenName"></param>
		/// <returns>True if it's </returns>
		public static bool IsExternalIncludeMacro(List<Token> Tokens)
		{
			if(Tokens.Count == 1 && Tokens[0].Text.StartsWith("FT_") && Tokens[0].Text.EndsWith("_H"))
			{
				return true;
			}
			else
			{
				return false;
			}
		}

		/// <summary>
		/// List of files whose includes are pinned to the file they are included from
		/// </summary>
		static readonly HashSet<string> PinnedFileNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
			"/Engine/Source/Runtime/Core/Public/GenericPlatform/GenericPlatform.h",
			"/Engine/Source/Runtime/Core/Public/Math/UnrealMatrix.h",
			"/Engine/Source/Runtime/Launch/Public/RequiredProgramMainCPPInclude.h",

			// Stuff which is pretty platform specific
			"/Engine/Source/Runtime/Core/Public/HAL/PThreadCriticalSection.h",
			"/Engine/Source/Runtime/Core/Public/HAL/PThreadEvent.h",
			"/Engine/Source/Runtime/Core/Private/HAL/PThreadRunnableThread.h",
			"/Engine/Source/Runtime/Core/Public/GenericPlatform/StandardPlatformString.h",
			"/Engine/Source/Runtime/Core/Public/GenericPlatform/MicrosoftPlatformString.h",
			"/Engine/Source/Runtime/Core/Public/Math/UnrealPlatformMathSSE.h",

			// Non standalone
			"/Engine/Source/Runtime/Core/Public/Math/UnrealMathSSE.h", // Needs to be included from UnrealMathUtility.h
			"/Engine/Source/Runtime/Core/Public/Math/UnrealMathVectorConstants.h", // Relies on being included from the middle of UnrealMathSSE.h
			"/Engine/Source/Runtime/Core/Public/Math/UnrealMathVectorCommon.h", // Needs to be included from UnrealMathUtility.h
			"/Engine/Source/Runtime/Core/Public/Misc/StringFormatArg.h",
			"/Engine/Source/Runtime/Core/Public/Math/TransformVectorized.h",
			"/Engine/Source/Runtime/Core/Public/Math/TransformNonVectorized.h",
			"/Engine/Source/Runtime/Core/Public/Containers/LockFreeVoidPointerListBase.h",
			"/Engine/Source/Runtime/Core/Public/Math/ScalarRegister.h",

			"/Engine/Source/Runtime/Core/Public/Templates/SharedPointerInternals.h",

			"/Engine/Source/Runtime/Core/Private/Internationalization/ICUText.h",
			"/Engine/Source/Runtime/Core/Private/Internationalization/LegacyText.h",

			"/Engine/Source/Runtime/Core/Public/Delegates/DelegateInstanceInterface.h",
			"/Engine/Source/Runtime/Core/Public/Delegates/DelegateInstancesImpl.h",
			"/Engine/Source/Runtime/Core/Public/Delegates/DelegateSignatureImpl.inl",
			"/Engine/Source/Runtime/Core/Public/Delegates/DelegateCombinations.h",

			// Compiler hangs after errors if it tries to compile with this separated from NumberFormattingRules.h
			"/Engine/Source/Runtime/Core/Public/Internationalization/NegativePercentOutputFormatting.h",
			"/Engine/Source/Runtime/Core/Public/Internationalization/PositivePercentOutputFormatting.h",
			"/Engine/Source/Runtime/Core/Public/Internationalization/PositiveCurrencyOutputFormatting.h",
			"/Engine/Source/Runtime/Core/Public/Internationalization/NegativeNumberOutputFormatting.h",

			// Dependency on something that includes it
			"/Engine/Source/Runtime/Core/Public/Delegates/DelegateCombinations.h", // Delegate.h
			"/Engine/Source/Runtime/Core/Public/Containers/LockFreeListImpl.h", // LockFreeList.h
			"/Engine/Source/Runtime/Core/Public/Stats/Stats2.h", // Stats.h
			"/Engine/Source/Runtime/Launch/Resources/VersionLocked.h", // Version.h
			"/Engine/Source/Runtime/RHI/Public/RHIResources.h",
			"/Engine/Source/Runtime/RHI/Public/DynamicRHI.h",
			"/Engine/Source/Runtime/RHI/Public/RHICommandList.h",
			"/Engine/Source/Runtime/RHI/Public/RHIUtilities.h",
			"/Engine/Source/Runtime/OpenGLDrv/Public/OpenGLShaderResources.h",

			// All this stuff in CoreTypes.h should never be expanded out
			"/Engine/Source/Runtime/Core/Public/Misc/Build.h",
			"/Engine/Source/Runtime/Core/Public/HAL/Platform.h",
			"/Engine/Source/Runtime/Core/Public/Windows/WindowsPlatform.h",
			"/Engine/Source/Runtime/Core/Public/Linux/LinuxPlatform.h",
			"/Engine/Source/Runtime/Core/Public/Windows/WindowsPlatformCompilerPreSetup.h",
			"/Engine/Source/Runtime/Core/Public/Linux/LinuxPlatformCompilerPreSetup.h",
			"/Engine/Source/Runtime/Core/Public/Windows/WindowsPlatformCompilerSetup.h",
			"/Engine/Source/Runtime/Core/Public/Linux/LinuxPlatformCompilerSetup.h",
			"/Engine/Source/Runtime/Core/Public/GenericPlatform/GenericPlatform.h",
			"/Engine/Source/Runtime/Core/Public/GenericPlatform/GenericPlatformCompilerSetup.h",
			"/Engine/Source/Runtime/Core/Public/GenericPlatform/GenericPlatformCompilerPreSetup.h",
			"/Engine/Source/Runtime/Core/Public/ProfilingDebugging/UMemoryDefines.h",
			"/Engine/Source/Runtime/Core/Public/Misc/CoreMiscDefines.h",
			"/Engine/Source/Runtime/Core/Public/Misc/CoreDefines.h",

			// Platform wrapper
			"/Engine/Source/Runtime/Online/SSL/Private/PlatformSslCertificateManager.h",

			// Weird Android multiple target platform through INL file stuff
			"/Engine/Source/Developer/Android/AndroidTargetPlatformControls/Private/AndroidTargetDevice.h",
			"/Engine/Source/Developer/Android/AndroidTargetPlatformControls/Private/AndroidTargetDeviceOutput.h",

			// Platform specific
			"/Engine/Source/Runtime/Slate/Public/Framework/Text/GenericPlatformTextField.h",

			// Base definitions for OpenGL3/4
			"/Engine/Source/Runtime/OpenGLDrv/Private/OpenGL.h",
			"/Engine/Source/Runtime/OpenGLDrv/Private/OpenGL3.h",
			"/Engine/Source/Runtime/OpenGLDrv/Private/OpenGL4.h",
			"/Engine/Source/Runtime/OpenGLDrv/Public/OpenGLUtil.h", // Requires external OpenGL headers
			"/Engine/Source/Runtime/OpenGLDrv/Private/OpenGLState.h", // Requires external OpenGL headers
			"/Engine/Source/Runtime/OpenGLDrv/Private/OpenGLResources.h", // Requires external OpenGL headers
		};

		/// <summary>
		/// List of files whose includes are pinned to the file they are included from
		/// </summary>
		static readonly HashSet<string> NotStandaloneFileNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
			"/Engine/Source/Runtime/Core/Public/Delegates/DelegateCombinations.h",
			"/Engine/Source/Runtime/Core/Public/Containers/LockFreeListImpl.h",
			"/Engine/Source/Runtime/Core/Public/Stats/Stats2.h",
			"/Engine/Source/Runtime/Launch/Resources/Version.h", // Doesn't actually need any includes, but we don't want build.h etc... included
			"/Engine/Source/Runtime/Launch/Resources/VersionLocked.h",
			"/Engine/Source/Runtime/Core/Private/Misc/EventPool.h",
			"/Engine/Source/Runtime/Core/Public/GenericPlatform/GenericPlatformCompilerPreSetup.h",
			"/Engine/Source/Runtime/Core/Public/Containers/LockFreeFixedSizeAllocator.h",
			"/Engine/Source/Runtime/Core/Public/Windows/COMPointer.h",
			"/Engine/Source/Runtime/Core/Public/Modules/ModuleVersion.h",
			"/Engine/Source/Runtime/Core/Resources/Windows/ModuleVersionResource.h",
			"/Engine/Source/Runtime/OpenGLDrv/Public/OpenGLUtil.h", // Requires external OpenGL headers
		};

		/// <summary>
		/// List of files whose includes are pinned to the file they are included from
		/// </summary>
		static readonly HashSet<string> AggregateFileNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
//			"/Engine/Source/Runtime/CoreUObject/Classes/Object.h",
			"/Engine/Source/Runtime/Slate/Public/SlateBasics.h",
			"/Engine/Source/Editor/UnrealEd/Public/UnrealEd.h"
		};

		/// <summary>
		/// List of files whose includes are pinned to the file they are included from
		/// </summary>
		static readonly HashSet<string> NonAggregateFileNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
			"/Engine/Source/Runtime/RHI/Public/RHI.h",
		};

		/// <summary>
		/// List of files which are ok to split into multiple fragments
		/// </summary>
		static readonly HashSet<string> AllowMultipleFragmentFileNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
			"/Engine/Source/Runtime/Sockets/Private/BSDSockets/SocketSubsystemBSD.h",
			"/Engine/Source/Runtime/Sockets/Private/BSDSockets/SocketsBSD.h",
			"/Engine/Plugins/Runtime/PacketHandlers/CompressionComponents/Oodle/Source/OodleHandlerComponent/Public/OodleHandlerComponent.h",

			// TODO check
			"/Engine/Source/Runtime/RHI/Public/RHI.h",
			"/Engine/Source/Runtime/Online/SSL/Private/PlatformSslCertificateManager.h",
		};

		/// <summary>
		/// Files which should not be treated as containing purely forward declarations, despite a "fwd.h" suffix
		/// </summary>
		static readonly HashSet<string> IgnoreFwdHeaders = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
			"/Engine/Source/Developer/DerivedDataCache/Public/DerivedDataSharedStringFwd.h", // invalid forward declaration - 'namespace UE::DerivedData'
			"/Engine/Source/Runtime/Core/Public/Internationalization/TextNamespaceFwd.h",
			"/Engine/Source/Editor/SceneOutliner/Public/SceneOutlinerFwd.h",
			"/Engine/Source/Editor/EditorFramework/Public/UnrealWidgetFwd.h", // error: invalid forward declaration - 'enum ECoordSystem'
			"/Engine/Source/Runtime/SlateCore/Public/Fonts/ShapedTextFwd.h", // Typedef isn't a forward declaration
			"/Engine/Source/Runtime/Slate/Public/Framework/Text/ShapedTextCacheFwd.h", // Typedef isn't a forward declaration
			"/Engine/Source/Runtime/MovieScene/Public/MovieSceneFwd.h",
            "/Engine/Source/Editor/SequencerCore/Public/SequencerCoreFwd.h", // invalid forward declaration - 'namespace UE'
			"/Engine/Source/Runtime/Core/Public/Async/TaskGraphFwd.h",  // warning: expected only include directives and text in forward declaration header in TaskGraphFwd.h
			"/Engine/Source/Runtime/Core/Public/Math/MathFwd.h", // invalid forward declaration - 'namespace UE::Math'
			"/Engine/Source/Runtime/Core/Public/Containers/ContainersFwd.h", // invalid forward declaration - 'template<> struct TIsContiguousContainer<Type> { static constexpr bool Value = true; };'
			"/Engine/Source/Runtime/Core/Public/Containers/StringFwd.h", // invalid forward declaration - 'template<> struct TIsContiguousContainer<Type> { static constexpr bool Value = true; };'
			"/Engine/Source/Runtime/Core/Public/Internationalization/StringTableCoreFwd.h", // Typedef isn't a forward declaration
			"/Engine/Source/Runtime/Experimental/Chaos/Public/Chaos/GeometryParticlesfwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/Chaos/ImplicitFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/Chaos/Island/IslandManagerFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/Chaos/ParticleHandleFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/Chaos/PBDRigidsEvolutionFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/Chaos/PBDSoftsEvolutionFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/Chaos/ShapeInstanceFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/Chaos/SpatialAccelerationFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/PhysicsProxy/JointConstraintProxyFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Experimental/Chaos/Public/PhysicsProxy/SingleParticlePhysicsProxyFwd.h", // invalid forward declaration - 'namespace Chaos'
			"/Engine/Source/Runtime/Interchange/Engine/Public/InterchangeEngineFwd.h", // invalid forward declaration - 'namespace UE'
			"/Engine/Plugins/Experimental/GameFeatures/Source/GameFeatures/Public/GameFeatureTypesFwd.h", //  invalid forward declaration - 'namespace GameFeaturePluginStatePrivate'
			"/Engine/Plugins/Experimental/StateGraph/Source/StateGraph/Public/StateGraphFwd.h", //  invalid forward declaration - 'namespace UE'
			"/Engine/Source/Runtime/Core/Public/Containers/VersePathFwd.h", // invalid forward declaration - 'namespace UE::Core'
			"/Engine/Source/Runtime/Experimental/Iris/Core/Public/Iris/ReplicationState/ReplicationStateFwd.h", // invalid forward declaration - 'namespace UE::Net'
			"/Engine/Source/Runtime/Experimental/Iris/Core/Private/Iris/ReplicationSystem/ObjectReferenceCacheFwd.h", // invalid forward declaration - 'namespace UE::Net'
			"/Engine/Source/Runtime/Online/HTTP/Public/HttpFwd.h", // error: invalid forward declaration - 'typedef TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> FHttpRequestPtr;'
			"/Engine/Restricted/NotForLicensees/Plugins/Online/OnlineSubsystemMcp/Source/Public/OnlineSubsystemMcpFwd.h", // expected only include directives and text in forward declaration header
			"/Engine/Source/Runtime/RHI/Public/RHIFwd.h", // invalid forward declaration - 'namespace ERHIFeatureLevel { enum Type : int; }'
			"/Engine/Source/Runtime/RenderCore/Public/RenderGraphFwd.h",
			"/Engine/Source/Runtime/Core/Public/Templates/SharedPointerFwd.h", // Has an enum as well
			"/Engine/Source/Runtime/Core/Public/Templates/SharedPointerFwd.h", // Has an enum as well
			"/Engine/Source/Runtime/Core/Public/Misc/OptionalFwd.h", // Has special struct
			"/Engine/Source/Runtime/CoreUObject/Public/UObject/ScriptDelegateFwd.h", // error: invalid forward declaration - 'typedef TScriptDelegate<FWeakObjectPtr> FScriptDelegate;'
			"/Engine/Source/Runtime/CoreUObject/Public/UObject/VerseTypesFwd.h", // error: invalid forward declaration - 'namespace Verse'
			"/Engine/Source/Runtime/CoreUObject/Public/UObject/WeakObjectPtrFwd.h", // error: invalid forward declaration - 'template<> struct TIsPODType<FWeakObjectPtr> { enum { Value = true }; };'
			"/Engine/Source/Runtime/Core/Public/UObject/WeakObjectPtrTemplatesFwd.h", // error: invalid forward declaration - 'template<class T> struct TIsPODType<TWeakObjectPtr<T> > { enum { Value = true }; };'
		};

		/// <summary>
		/// Gets the flags for a new source file
		/// </summary>
		/// <param name="File">The workspace file being read</param>
		/// <returns>Flags for the corresponding source file</returns>
		public static SourceFileFlags GetSourceFileFlags(WorkspaceFile File)
		{
			string NormalizedPath = File.NormalizedPathFromBranchRoot;
			if(NormalizedPath == null || IsExternalHeaderPath(NormalizedPath))
			{
				return SourceFileFlags.Pinned | SourceFileFlags.External;
			}

			SourceFileFlags Flags = SourceFileFlags.Standalone;
			if(NormalizedPath.EndsWith(".inl") || NormalizedPath.EndsWith(".inc") || NormalizedPath.EndsWith(".generated.h"))
			{
				Flags &= ~SourceFileFlags.Standalone;
				Flags |= SourceFileFlags.Pinned;
			}
			if(NormalizedPath.IndexOf("/public/") != -1 || NormalizedPath.IndexOf("/classes/") != -1)
			{
				Flags |= SourceFileFlags.Public;
			}
			if(NormalizedPath.IndexOf("/intermediate/") != -1)
			{
				if (NormalizedPath.EndsWith("ispc.generated.h"))
				{
					Flags |= SourceFileFlags.GeneratedHeader | SourceFileFlags.Public;
				}
				else if (NormalizedPath.EndsWith(".generated.h"))
				{
					Flags |= SourceFileFlags.GeneratedHeader | SourceFileFlags.Inline | SourceFileFlags.Public;
				}
				else if(NormalizedPath.EndsWith("classes.h"))
				{
					Flags |= SourceFileFlags.GeneratedClassesHeader | SourceFileFlags.Public;
				}
				else if(NormalizedPath.EndsWith(".gen.h") && NormalizedPath.IndexOf("/vni/") != -1)
				{
					Flags &= ~SourceFileFlags.Standalone;
					Flags |= SourceFileFlags.Pinned | SourceFileFlags.GeneratedHeader | SourceFileFlags.Public | SourceFileFlags.AllowMultipleFragments;
				}
			}
			if(NormalizedPath.EndsWith(".cpp") || NormalizedPath.IndexOf("/windows/") != -1 || NormalizedPath.IndexOf("/linux/") != -1)
			{
				Flags |= SourceFileFlags.Pinned;
			}

			if (NormalizedPath.EndsWith("fwd.h") && !IgnoreFwdHeaders.Contains(NormalizedPath))
			{
				Flags |= SourceFileFlags.FwdHeader;
			}

			if(NormalizedPath.EndsWith("defineupropertymacros.h"))
			{
				Flags |= SourceFileFlags.Inline;
			}

			if (PinnedFileNames.Contains(NormalizedPath))
			{
				Flags = (Flags | SourceFileFlags.Pinned) & ~SourceFileFlags.Standalone;
			}
			if(NotStandaloneFileNames.Contains(NormalizedPath))
			{
				Flags &= ~SourceFileFlags.Standalone;
			}
			if(AggregateFileNames.Contains(NormalizedPath))
			{
				Flags |= SourceFileFlags.Aggregate;
			}
			if(AllowMultipleFragmentFileNames.Contains(NormalizedPath))
			{
				Flags |= SourceFileFlags.AllowMultipleFragments;
			}
			if(ShouldIgnoreExports(NormalizedPath))
			{
				Flags |= SourceFileFlags.IgnoreExportedSymbols;
			}
			return Flags;
		}

		/// <summary>
		/// Apply any additional markup to files in the branch
		/// </summary>
		/// <param name="BranchRoot">Root directory for the branch</param>
		public static void ApplyAdditionalRules(DirectoryReference BranchRoot)
		{
			SourceFile CoreTypesFile = Workspace.GetFile(FileReference.Combine(BranchRoot, "Engine\\Source\\Runtime\\Core\\Public\\CoreTypes.h")).ReadSourceFile();
			CoreTypesFile.Flags |= SourceFileFlags.Monolithic | SourceFileFlags.IsCoreTypes;

			SourceFile CoreMinimalFile = Workspace.GetFile(FileReference.Combine(BranchRoot, "Engine\\Source\\Runtime\\Core\\Public\\CoreMinimal.h")).ReadSourceFile();
			CoreMinimalFile.Flags |= SourceFileFlags.Monolithic | SourceFileFlags.IsCoreMinimal;

			AddCounterpart(BranchRoot, "Engine\\Source\\Runtime\\Core\\Public\\Windows\\AllowWindowsPlatformTypes.h", "Engine\\Source\\Runtime\\Core\\Public\\Windows\\HideWindowsPlatformTypes.h");
			AddCounterpart(BranchRoot, "Engine\\Source\\Runtime\\Core\\Public\\Windows\\AllowWindowsPlatformAtomics.h", "Engine\\Source\\Runtime\\Core\\Public\\Windows\\HideWindowsPlatformAtomics.h");
			AddCounterpart(BranchRoot, "Engine\\Source\\Runtime\\Core\\Public\\Windows\\PreWindowsApi.h", "Engine\\Source\\Runtime\\Core\\Public\\Windows\\PostWindowsApi.h");
			AddCounterpart(BranchRoot, "Engine\\Plugins\\Runtime\\OpenCV\\Source\\OpenCVHelper\\Public\\PreOpenCVHeaders.h", "Engine\\Plugins\\Runtime\\OpenCV\\Source\\OpenCVHelper\\Public\\PostOpenCVHeaders.h");
			AddCounterpart(BranchRoot, "Engine\\Plugins\\Media\\AjaMedia\\Source\\AjaMedia\\Private\\AjaMediaAllowPlatformTypes.h", "Engine\\Plugins\\Media\\AjaMedia\\Source\\AjaMedia\\Private\\AjaMediaHidePlatformTypes.h");
			AddCounterpart(BranchRoot, "Engine\\Plugins\\Enterprise\\DatasmithC4DImporter\\Source\\DatasmithC4DTranslator\\Private\\DatasmithC4DMelangeSDKEnterGuard.h", "Engine\\Plugins\\Enterprise\\DatasmithC4DImporter\\Source\\DatasmithC4DTranslator\\Private\\DatasmithC4DMelangeSDKLeaveGuard.h");
		}

		/// <summary>
		/// Adds one file as a counterpart to the other
		/// </summary>
		/// <param name="BranchRoot">Root directory for the branch</param>
		/// <param name="FirstFileName">Path to the first file from the branch root directory</param>
		/// <param name="SecondFileName">Path to the second file from the branch root directory</param>
		static void AddCounterpart(DirectoryReference BranchRoot, string FirstFileName, string SecondFileName)
		{
			SourceFile FirstFile = Workspace.GetFile(FileReference.Combine(BranchRoot, FirstFileName)).ReadSourceFile();
			SourceFile SecondFile = Workspace.GetFile(FileReference.Combine(BranchRoot, SecondFileName)).ReadSourceFile();
			FirstFile.SetCounterpart(SecondFile);
		}

		/// <summary>
		/// Override to allow a block of markup to be be different between two derivations
		/// </summary>
		/// <param name="File">File being preprocessed</param>
		/// <param name="MarkupIdx">Index of the markup object to consider</param>
		/// <returns>True to ignore the different derivations</returns>
		public static bool AllowDifferentMarkup(SourceFile File, int MarkupIdx)
		{
			PreprocessorMarkup Markup = File.Markup[MarkupIdx];
			if(Markup.Type == PreprocessorMarkupType.Text && Markup.EndLocation.LineIdx == Markup.Location.LineIdx + 1 && File.Text.Lines[Markup.Location.LineIdx].Contains("friend") && File.Text.Lines[Markup.Location.LineIdx].Contains("Z_Construct_"))
			{
				return true;
			}
			if(Markup.Type == PreprocessorMarkupType.Define && Markup.Tokens[0].Text == "ONLINE_LOG_PREFIX")
			{
				return true;
			}			
			if(Markup.Type == PreprocessorMarkupType.Define && (Markup.Tokens[0].Text == "UE_DEPRECATED_FORGAME"))
			{
				return true;
			}
			if (Markup.Type == PreprocessorMarkupType.Define && Markup.Tokens[0].Text == "UE_DEPRECATED_FORENGINE")
			{
				return true;
			}
			if (Markup.Type == PreprocessorMarkupType.Undef && Markup.Tokens[0].Text == "TEXT")
			{
				return true;
			}
			if(Markup.Type == PreprocessorMarkupType.Else || Markup.Type == PreprocessorMarkupType.Endif)
			{
				return true;
			}
			// Start of RigLogic exclusions
			if (Markup.Type == PreprocessorMarkupType.Define && (Markup.Tokens[0].Text == "RLAPI" || Markup.Tokens[0].Text == "DNAAPI" || Markup.Tokens[0].Text == "TRIOAPI" || Markup.Tokens[0].Text == "SCAPI" || Markup.Tokens[0].Text == "PMAAPI" || Markup.Tokens[0].Text == "RAFAPI" || Markup.Tokens[0].Text == "GSAPI" || Markup.Tokens[0].Text == "DNACAPI"))
			{
				return true;
			}
			if (Markup.Type == PreprocessorMarkupType.Elif && Markup.Tokens.Count == 4 && (Markup.Tokens[2].Text == "RL_SHARED" || Markup.Tokens[2].Text == "GS_SHARED" || Markup.Tokens[2].Text == "DNAC_SHARED"))
			{
				return true;
			}
			if (Markup.Type == PreprocessorMarkupType.If && Markup.Tokens.Count == 4 && Markup.Tokens[2].Text == "_MSC_VER")
			{
				return true;
			}
			if (Markup.Type == PreprocessorMarkupType.Define && Markup.Tokens.Count == 8 && Markup.Tokens[0].Text == "FORCE_INLINE")
			{
				return true;
			}
			// End of RigLogic exclusions
			if((File.Flags & SourceFileFlags.External) != 0)
			{
				return true;
			}
			return false;
		}

		static readonly string[] PathsToIgnoreForOldStyleHeaders = new string[] {
			"/Engine/Source/Runtime/Navmesh/Public/DebugUtils/",
			"/Engine/Source/Runtime/Navmesh/Public/Detour/",
			"/Engine/Source/Runtime/Navmesh/Public/DetourCrowd/",
			"/Engine/Source/Runtime/Navmesh/Public/DetourTileCache/",
			"/Engine/Source/Runtime/Navmesh/Public/Recast/",

			"/Engine/Plugins/Compression/OodleNetwork/",
			"/Engine/Source/Developer/TextureFormatOodle/",
			"/Engine/Plugins/Media/BinkMedia/Source/",
			"/Engine/Source/Runtime/OodleDataCompression/Sdks/",

			"/Engine/Plugins/Animation/ControlRigSpline/Source/ControlRigSpline/ThirdParty/",
			"/Engine/Plugins/Runtime/nDisplay/ThirdParty/",
			"/Engine/Plugins/Runtime/ResonanceAudio/Source/ResonanceAudio/Private/ResonanceAudioLibrary/",
			"/Engine/Source/Runtime/Experimental/Voronoi/",

			"/Engine/Plugins/Animation/ControlRig/Source/ControlRig/ThirdParty/AHEasing/AHEasing/easing.h",
			"/Engine/Plugins/Animation/RigLogic/Source/RigLogicLib/Private/dna/utils/Extd.h",
			"/Engine/Plugins/Animation/RigLogic/Source/RigLogicLib/Private/riglogic/utils/Extd.h",
			"/Engine/Plugins/Animation/RigLogic/Source/RigLogicLib/Public/dna/types/ArrayView.h",
			"/Engine/Source/Runtime/Core/Public/Experimental/Containers/FAAArrayQueue.h",
			"/Engine/Source/Runtime/Core/Public/Hash/CityHash.h",
			"/Engine/Source/Runtime/Core/Public/MemPro/MemPro.h",
			"/Engine/Source/Runtime/CUDA/Source/Public/CudaWrapper.h",
			"/Engine/Plugins/Runtime/GeometryProcessing/Source/GeometryAlgorithms/Private/ThirdParty/xatlas/xatlas.h",

			"/Engine/Restricted/",
			"/Engine/Shaders/Shared/RayTracingBuiltInResources.h",
			"/Engine/Source/Runtime/Symslib/syms/",
		};

		/// <summary>
		/// Whether to ignore old-style header guards
		/// </summary>
		/// <param name="NormalizedPath">Path to the file</param>
		/// <returns>True to ignore the different derivations</returns>
		public static bool IgnoreOldStyleHeaderGuards(FileReference Location)
		{
			string LocationString = Location.ToString().Replace("\\", "/");
			return PathsToIgnoreForOldStyleHeaders.Any(x => LocationString.Contains(x));
		}

        /// <summary>
        /// Allow overriding whether a symbol should be forward-declared
        /// </summary>
        /// <param name="Symbol"></param>
        /// <returns></returns>
        public static bool AllowSymbol(string Name)
        {
            if(Name == "FNode" || Name == "FFunctionExpression" || Name == "ITextData" || Name == "Rect")
            {
                return false;
            }
            return true;
        }


		static readonly string[] PathsToIgnoreConflictingSymbols = new string[] {
			"/Engine/Source/Runtime/Core/Public/Async/TaskGraphInterfaces.h",
			"/Engine/Source/Runtime/Core/Public/Async/TaskGraphFwd.h",
			"/Engine/Source/Runtime/Core/Public/Delegates/DelegateAccessHandler.h",
			"/Engine/Plugins/Runtime/RigVM/Source/RigVM/Public/RigVMCore/RigVMTypeIndex.h",
		};

		/// <summary>
		/// Returns true if the file given should be included in report about conflicting symbols.
		/// </summary>
		/// <param name="Location">The file reference to check for inclusion</param>
		/// <returns>True to report conclicts for this file</returns>
		public static bool ReportConflictingSymbolsForFile(FileReference Location)
		{
			string LocationString = Location.ToString().Replace("\\", "/");
			return !PathsToIgnoreConflictingSymbols.Any(x => LocationString.Contains(x));
		}
	}
}
