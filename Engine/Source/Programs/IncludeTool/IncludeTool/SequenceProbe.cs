// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;
using System.Diagnostics;
using IncludeTool.Support;
using EpicGames.Core;

namespace IncludeTool
{
	/// <summary>
	/// Type of probe to create
	/// </summary>
	enum SequenceProbeType
	{
		/// <summary>
		/// Compile the sequence without optimizing to ensure that it is valid
		/// </summary>
		Verify,

		/// <summary>
		/// Find the minimal set of dependencies for the fragments
		/// </summary>
		Optimize,
	}

	/// <summary>
	/// Return code for a probe iteration
	/// </summary>
	enum SequenceProbeResult
	{
		/// <summary>
		/// The worker failed with a fatal error
		/// </summary>
		Failed,

		/// <summary>
		/// The worker failed due to a potentially intermittent error
		/// </summary>
		FailedAllowRetry,

		/// <summary>
		/// The worker was updated
		/// </summary>
		Updated,

		/// <summary>
		/// The fragment has been optimized
		/// </summary>
		Completed,
	}

	/// <summary>
	/// Worker class for finding dependencies of a given source file
	/// </summary>
	class SequenceProbe : IDisposable
	{
		/// <summary>
		/// What mode to run this worker in
		/// </summary>
		public readonly SequenceProbeType Type;

		/// <summary>
		/// The full list of fragments from the root of the tree
		/// </summary>
		public readonly SourceFragment[] Fragments;

		/// <summary>
		/// The fragment we're finding dependencies for
		/// </summary>
		public readonly SourceFragment LastFragment;

		/// <summary>
		/// The current task state
		/// </summary>
		SequenceWorker Worker;

		/// <summary>
		/// Instance of the executing task
		/// </summary>
		ManagedTask ActiveInstance;

		/// <summary>
		/// Output directory for intermediate files
		/// </summary>
		DirectoryReference IntermediateDir;

		/// <summary>
		/// Unique name assigned to 
		/// </summary>
		string UniqueName;

		/// <summary>
		/// Path to the file storing the serialized task
		/// </summary>
		FileReference TaskStateFile;

		/// <summary>
		/// Path to the summary log file
		/// </summary>
		FileReference SummaryLogFile;

		/// <summary>
		/// Path to the compiler log file
		/// </summary>
		FileReference CompileLogFile;

		/// <summary>
		/// 
		/// </summary>
		HashSet<SourceFragment> WarnedMissingDependencies = new HashSet<SourceFragment>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Type">The type of probe to create</param>
		/// <param name="Node">The node to optimize</param>
		/// <param name="IntermediateDir">Directory for intermediate files</param>
		/// <param name="UniqueName">Unique name prefix for all temporary files</param>
		public SequenceProbe(SequenceProbeType Type, SourceFragment[] Fragments, Tuple<int, SourceFile>[] IncludeHistory, CompileEnvironment CompileEnvironment, DirectoryReference IntermediateDir, IEnumerable<DirectoryReference> ExtraSystemIncludePaths, string UniqueName)
		{
			this.Type = Type;
			this.IntermediateDir = IntermediateDir;
			this.Fragments = Fragments;
			this.LastFragment = Fragments[Fragments.Length - 1];
			this.UniqueName = UniqueName;
			this.TaskStateFile = FileReference.Combine(IntermediateDir, UniqueName + ((Type == SequenceProbeType.Verify)? ".verify.state" : ".state"));
			this.SummaryLogFile = FileReference.Combine(IntermediateDir, UniqueName + ".summary.txt");
			this.CompileLogFile = FileReference.Combine(IntermediateDir, UniqueName + ".compile.txt");

			// Get the file to use for trying to compile different permutations
			FileReference PermutationFile = FileReference.Combine(IntermediateDir, UniqueName + ".permutation");

			// Create the response file
			FileReference ResponseFile = FileReference.Combine(IntermediateDir, UniqueName + ".response");
			CompileEnvironment WorkerCompileEnvironment = new CompileEnvironment(CompileEnvironment);
			if(WorkerCompileEnvironment.CompilerType == CompilerType.Clang)
			{
				WorkerCompileEnvironment.Options.Add(new CompileOption("-o", FileReference.Combine(IntermediateDir, UniqueName + ".o").FullName.Replace('\\', '/')));
			}
			else
			{
				WorkerCompileEnvironment.Options.RemoveAll(x => x.Name == "/Z7" || x.Name == "/Zi" || x.Name == "/ZI");
				WorkerCompileEnvironment.Options.Add(new CompileOption("/Fo", FileReference.Combine(IntermediateDir, UniqueName + ".obj").FullName));
				WorkerCompileEnvironment.Options.Add(new CompileOption("/WX", null));
			}
			WorkerCompileEnvironment.WriteResponseFile(ResponseFile, PermutationFile);
			string ResponseFileDigest = Utility.ComputeDigest(ResponseFile);

			// Keep track of the include stack, so we can format the flat fragment list with context
			int IncludeHistoryIdx = 0;
			List<SourceFile> IncludeStack = new List<SourceFile>();

			// Create the script for the probe
			List<Tuple<int, string>> Lines = new List<Tuple<int, string>>();
			for(int Idx = 0; Idx < Fragments.Length; Idx++)
			{
				SourceFragment Fragment = Fragments[Idx];

				// Figure out which tag it's bound to
				int Tag = (Fragment.File.Counterpart == null)? Idx : -1;

				// Update the stack for new includes
				while(IncludeHistoryIdx < IncludeHistory.Length && IncludeHistory[IncludeHistoryIdx].Item1 == Idx)
				{
					if(IncludeHistory[IncludeHistoryIdx].Item2 == null)
					{
						SourceFile IncludeFile = IncludeStack[IncludeStack.Count - 1];
						IncludeStack.RemoveAt(IncludeStack.Count - 1);
						Lines.Add(new Tuple<int, string>(Tag, String.Format("{0}// END INCLUDE {1}", new string(' ', IncludeStack.Count * 4), IncludeFile.Location.FullName)));
						IncludeHistoryIdx++;
					}
					else if(IncludeHistoryIdx + 1 < IncludeHistory.Length && IncludeHistory[IncludeHistoryIdx + 1].Item2 == null)
					{
						IncludeHistoryIdx += 2;
					}
					else
					{
						SourceFile IncludeFile = IncludeHistory[IncludeHistoryIdx].Item2;
						Lines.Add(new Tuple<int, string>(Tag, String.Format("{0}// INCLUDE {1}", new string(' ', IncludeStack.Count * 4), IncludeFile.Location.FullName)));
						IncludeStack.Add(IncludeFile);
						IncludeHistoryIdx++;
					}
				}

				// Get the indent at this point
				string Indent = new string(' ', (IncludeStack.Count - 0) * 4);

				// Write out the forward declarations for every symbol referenced in this fragment. We don't want false dependencies caused by forward declarations in other fragments
				// if the heuristic for detecting when to use them doesn't work.
				if((Fragment.File.Flags & SourceFileFlags.TranslationUnit) == 0)
				{
					foreach(KeyValuePair<Symbol, SymbolReferenceType> ReferencedSymbol in Fragment.ReferencedSymbols)
					{
						if(!String.IsNullOrEmpty(ReferencedSymbol.Key.ForwardDeclaration))
						{
							Lines.Add(Tuple.Create(Tag, Indent + ReferencedSymbol.Key.ForwardDeclaration));
						}
					}
				}

				// Some Clang/GCC system header wrappers require including as system includes in order to make the #include_next directive work
				DirectoryReference BaseSystemIncludePath = ExtraSystemIncludePaths.FirstOrDefault(x => Fragment.Location.IsUnderDirectory(x));
				if(BaseSystemIncludePath != null)
				{
					Lines.Add(Tuple.Create(Tag, String.Format("{0}#include <{1}>", Indent, Fragment.Location.MakeRelativeTo(BaseSystemIncludePath))));
				}
				else
				{
					Lines.Add(Tuple.Create(Tag, String.Format("{0}#include \"{1}\"", Indent, Fragment.Location.FullName)));
				}
			}

			// Create the new task
			string[] FragmentFileNames = Fragments.Select(Fragment => Fragment.Location.FullName).ToArray();
			string[] FragmentDigests = Fragments.Select(Fragment => Fragment.Digest ?? "").ToArray();
			Worker = new SequenceWorker(PermutationFile.FullName, ResponseFile.FullName, ResponseFileDigest, FragmentFileNames, FragmentDigests, Lines.ToArray(), CompileEnvironment.Compiler.FullName);
			AddDependency(Worker, Fragments, Fragments.Length - 1);

			// Log the referenced symbols
			if(LastFragment.ReferencedSymbols.Count > 0)
			{
				List<string> ReferenceLog = new List<string>();
				ReferenceLog.Add(String.Format("Referenced symbols for {0}:", LastFragment.Location));
				foreach(KeyValuePair<Symbol, SymbolReferenceType> Pair in LastFragment.ReferencedSymbols.OrderBy(x => x.Key.Type).ThenBy(x => x.Key.Name))
				{
					Symbol ReferencedSymbol = Pair.Key;
					ReferenceLog.Add(String.Format("    {0}: {1} ({2}; {3})", ReferencedSymbol.Type.ToString(), ReferencedSymbol.Name, Pair.Value.ToString(), ReferencedSymbol.Fragment.Location));
				}
				ReferenceLog.Add("");
				Worker.SummaryLog.InsertRange(0, ReferenceLog);
			}

			// Check to see if an existing version of the task exists which we can continue
			if(Type == SequenceProbeType.Optimize && FileReference.Exists(TaskStateFile))
			{
				// Try to read the old task
				SequenceWorker OldWorker;
				try
				{
					OldWorker = SequenceWorker.Deserialize(TaskStateFile.FullName);
				}
				catch(Exception)
				{
					OldWorker = null;
				}

				// If it succeeded, compare it to the new task
				if(OldWorker != null)
				{
					SequenceWorker NewWorker = Worker;

					// Create a list of fragments which can be ignored, because they're already determined to be not part of the result for the old task
					HashSet<string> IgnoreFragments = new HashSet<string>();
					for(int Idx = 0; Idx < OldWorker.FragmentCount; Idx++)
					{
						if(!OldWorker.KnownDependencies.Contains(Idx) && Idx >= OldWorker.RemainingFragmentCount)
						{
							IgnoreFragments.Add(OldWorker.FragmentFileNames[Idx]);
						}
					}
					IgnoreFragments.ExceptWith(NewWorker.KnownDependencies.Select(x => NewWorker.FragmentFileNames[x]));

					// Compute digests for the old and new tasks
					string OldDigest = CreateDigest(OldWorker, IgnoreFragments);
					string NewDigest = CreateDigest(NewWorker, IgnoreFragments);
					if(OldDigest == NewDigest)
					{
						// Build a map of fragment file names to their new index
						Dictionary<string, int> FragmentFileNameToNewIndex = new Dictionary<string, int>();
						for(int Idx = 0; Idx < FragmentFileNames.Length; Idx++)
						{
							string FragmentFileName = FragmentFileNames[Idx];
							FragmentFileNameToNewIndex[FragmentFileName] = Idx;
						}

						// Add known dependencies to the new worker
						foreach(int OldKnownDependency in OldWorker.KnownDependencies)
						{
							string OldFragmentFileName = OldWorker.FragmentFileNames[OldKnownDependency];
							int NewKnownDependency = FragmentFileNameToNewIndex[OldFragmentFileName];
							NewWorker.KnownDependencies.Add(NewKnownDependency);
						}

						// Update the remaining count. All these fragments must match, because they're not part of the ignore list.
						NewWorker.RemainingFragmentCount = OldWorker.RemainingFragmentCount;
					}
				}
			}

			// If this is a cpp file, make sure we have a dependency on the header file with the same name. It may specify linkage for the functions we declare.
			FileReference MainFileLocation = Fragments[Fragments.Length - 1].File.Location;
			if(MainFileLocation.HasExtension(".cpp"))
			{
				string HeaderFileName = Path.ChangeExtension(MainFileLocation.GetFileName(), ".h");
				for(int FragmentIdx = 0; FragmentIdx < Fragments.Length; FragmentIdx++)
				{
					if(String.Compare(Fragments[FragmentIdx].File.Location.GetFileName(), HeaderFileName, true) == 0)
					{
						AddDependency(Worker, Fragments, FragmentIdx);
					}
				}
			}

			// Update the finished fragment if we're done, otherwise clear out all the intermediate files
			if(Worker.RemainingFragmentCount == 0)
			{
				SetCompletedDependencies();
			}
			else
			{
				Worker.Serialize(TaskStateFile.FullName);

				FileReference.Delete(PermutationFile);
				FileReference.Delete(SummaryLogFile);
				FileReference.Delete(CompileLogFile);
			}
		}

		/// <summary>
		/// Adds a known dependency to the given task
		/// </summary>
		/// <param name="Worker">The task to add the dependency to</param>
		/// <param name="Fragments">The array of fragments</param>
		/// <param name="FragmentIdx">Index of the dependency</param>
		static void AddDependency(SequenceWorker Worker, SourceFragment[] Fragments, int FragmentIdx)
		{
			// Add the dependency
			Worker.KnownDependencies.Add(FragmentIdx);

			// Check for any macro definitions that the given fragment depends on
			SourceFragment Fragment = Fragments[FragmentIdx];
			foreach(Symbol ReferencedSymbol in Fragment.ReferencedSymbols.Keys)
			{
				if(ReferencedSymbol.Type == SymbolType.Macro)
				{
					int ReferencedFragmentIdx = Array.IndexOf(Fragments, ReferencedSymbol.Fragment);
					if(ReferencedFragmentIdx != -1 && ReferencedFragmentIdx < FragmentIdx)
					{
						Worker.KnownDependencies.Add(ReferencedFragmentIdx);
					}
				}
			}

			// Force dependencies onto pinned fragments. This can make a difference when templated functions are implemented in pinned headers
			// but declared in another (eg. LegacyText.h). The file will compile fine without the implementations being present, expecting them to be linked
			// into another object file, but fail when they are included due to being pinned later because MSVC only parses the token stream then.
			for(int MarkupIdx = Fragment.MarkupMin; MarkupIdx < Fragment.MarkupMax; MarkupIdx++)
			{
				PreprocessorMarkup Markup = Fragment.File.Markup[MarkupIdx];
				if(Markup.Type == PreprocessorMarkupType.Include && Markup.IncludedFile != null && (Markup.IncludedFile.Flags & SourceFileFlags.Pinned) != 0 && Markup.IncludedFile.Counterpart == null)
				{
					foreach(SourceFragment PinnedFragment in Markup.IncludedFile.Fragments)
					{
						int PinnedFragmentIdx = Array.IndexOf(Fragments, PinnedFragment);
						if(PinnedFragmentIdx != -1 && PinnedFragmentIdx < FragmentIdx)
						{
							AddDependency(Worker, Fragments, PinnedFragmentIdx);
						}
					}
				}
			}
		}

		/// <summary>
		/// Creates a string representing a context-free representation of the task, for using in comparing whether the task parameters have changed.
		/// </summary>
		/// <param name="Worker">The task to generate a digest for</param>
		/// <param name="IgnoreFragments">Set of fragments which can be ignored</param>
		/// <returns>String digest of the task</returns>
		static string CreateDigest(SequenceWorker Worker, HashSet<string> IgnoreFragments)
		{
			StringWriter Writer = new StringWriter();

			Writer.WriteLine("ResponseFile: {0}", Worker.ResponseFileDigest);
			Writer.WriteLine("Fragments:");
			for(int Idx = 0; Idx < Worker.FragmentFileNames.Length; Idx++)
			{
				string FragmentFileName = Worker.FragmentFileNames[Idx];
				if(!IgnoreFragments.Contains(FragmentFileName))
				{
					Writer.WriteLine("    {0}={1}", FragmentFileName, Worker.FragmentDigests[Idx]);
				}
			}
			Writer.WriteLine("Script:");
			foreach(Tuple<int, string> Line in Worker.Lines)
			{
				if (Line.Item1 < 0)
				{
					Writer.WriteLine("    None={0}", Line.Item2);
				}
				else if(!IgnoreFragments.Contains(Worker.FragmentFileNames[Line.Item1]))
				{
					Writer.WriteLine("    {0}={1}", Worker.FragmentFileNames[Line.Item1], Line.Item2);
				}
			}

			return Writer.ToString();
		}

		/// <summary>
		/// Dispose of any executing task object
		/// </summary>
		public void Dispose()
		{
			if(ActiveInstance != null)
			{
				ActiveInstance.Dispose();
				ActiveInstance = null;
			}
		}

		/// <summary>
		/// Checks whether the worker is complete
		/// </summary>
		public bool IsComplete
		{
			get { return Worker.RemainingFragmentCount == 0; }
		}

		/// <summary>
		/// The number of fragments in the node being optimized
		/// </summary>
		public int FragmentCount
		{
			get { return Fragments.Length; }
		}

		/// <summary>
		/// The number of completed fragments in the node being optimized
		/// </summary>
		public int CompletedFragmentCount
		{
			get { return Fragments.Length - Worker.RemainingFragmentCount; }
		}

		/// <summary>
		/// Update the list of known dependencies to include dependencies of our already known dependencies
		/// </summary>
		/// <returns>True if the dependencies were updated, false if a dependency was encountered which is not available to this fragment</returns>
		void UpdateDependencies(LineBasedTextWriter Log)
		{
			// Update the task to account for any new dependencies that may have been found. Loop back through all the fragments so we include
			// dependencies we find along the way.
			for(int FragmentIdx = Worker.FragmentCount - 1; FragmentIdx > 0; FragmentIdx--)
			{
				SourceFragment Fragment = Fragments[FragmentIdx];
				if(Worker.KnownDependencies.Contains(FragmentIdx) && Fragment.Dependencies != null)
				{
					// Include the dependencies of this fragment in our dependencies list
					List<SourceFragment> MissingDependencies = new List<SourceFragment>();
					foreach(SourceFragment Dependency in Fragment.Dependencies)
					{
						int DependencyIdx = Array.IndexOf(Fragments, Dependency);
						if(DependencyIdx == -1 || DependencyIdx > FragmentIdx)
						{
							MissingDependencies.Add(Dependency);
						}
						else
						{
							Worker.KnownDependencies.Add(DependencyIdx);
						}
					}

					// If any were missing (because they were calculated by a different derivation), provide a useful diagnostic. Otherwise re-queue the worker
					// to find the next dependency.
					if(MissingDependencies.Count > 0 && MissingDependencies.Any(x => !WarnedMissingDependencies.Contains(x)))
					{
						StringBuilder Message = new StringBuilder();
						Log.WriteLine("warning: could not find dependencies of '{0}':", Fragment.Location.FullName);
						foreach(SourceFragment MissingDependency in MissingDependencies)
						{
							Log.WriteLine("    {0}", MissingDependency.Location);
						}
						Log.WriteLine("In fragments for '{0}':", LastFragment.Location);
						for(int Idx = 0; Idx < Worker.RemainingFragmentCount; Idx++)
						{
							Log.WriteLine("    {0,3}: {1}", Idx, Fragments[Idx].Location.FullName);
						}
						for(int Idx = Worker.RemainingFragmentCount; Idx < Fragments.Length; Idx++)
						{
							Log.WriteLine("    {0,3}: ({1})", Idx, Fragments[Idx].Location.FullName);
						}
						WarnedMissingDependencies.UnionWith(MissingDependencies);
					}
				}
			}
		}

		/// <summary>
		/// Start this worker
		/// </summary>
		public void Start(LineBasedTextWriter Log)
		{
			// Make sure there's not already a managed task in process
			if(ActiveInstance != null)
			{
				throw new Exception("Cannot start a new worker while a managed process is already active");
			}

			// Create the new instance
			if(Type == SequenceProbeType.Optimize)
			{
				// Update our dependencies
				UpdateDependencies(Log);

				// Write the task to disk
				Worker.Serialize(TaskStateFile.FullName);

				// Construct the new managed task, and start it. If the task is already complete, create a dummy thread which just returns immediately.
				ActiveInstance = new ManagedTaskThread(Writer => FindNextDependency(TaskStateFile.FullName, Writer));
			}
			else if(Type == SequenceProbeType.Verify)
			{
				// Write the task to disk
				Worker.Serialize(TaskStateFile.FullName);

				// Construct the new managed task, and start it. If the task is already complete, create a dummy thread which just returns immediately.
				ActiveInstance = new ManagedTaskThread(Writer => Verify(TaskStateFile.FullName, Writer));
			}
			else
			{
				throw new NotImplementedException();
			}
			ActiveInstance.Start();
		}

		/// <summary>
		/// Checks that the given permutation compiles
		/// </summary>
		/// <param name="StateFile">Path to the state file</param>
		/// <param name="Writer">Writer for any messages</param>
		/// <returns>Zero on success</returns>
		static int Verify(string StateFile, LineBasedTextWriter Writer)
		{
			SequenceWorker Task = SequenceWorker.Deserialize(StateFile);
			Task.Verify(Writer);
			Task.Serialize(StateFile + ".out");
			return 0;
		}

		/// <summary>
		/// Runs the FindDependencyTask with the given state file
		/// </summary>
		/// <param name="StateFile">Path to the state file</param>
		/// <param name="Writer">Writer for any messages</param>
		/// <returns>Zero on success</returns>
		static int FindNextDependency(string StateFile, LineBasedTextWriter Writer)
		{
			SequenceWorker Task = SequenceWorker.Deserialize(StateFile);
			Task.FindNextDependency(Writer);
			Task.Serialize(StateFile + ".out");
			return 0;
		}

		/// <summary>
		/// Wait for this worker to complete
		/// </summary>
		/// <param name="Writer">Writer for log output</param>
		/// <returns>Return code from the thread</returns>
		public SequenceProbeResult Join(LineBasedTextWriter Writer)
		{
			// Finish the task instance
			BufferedTextWriter BufferedWriter = new BufferedTextWriter();
			int ExitCode = ActiveInstance.Join(BufferedWriter);
			ActiveInstance.Dispose();
			ActiveInstance = null;

			// Read the new state
			FileReference OutputFile = new FileReference(TaskStateFile.FullName + ".out");
			SequenceWorker NewWorker = SequenceWorker.Deserialize(OutputFile.FullName);
			FileReference.Delete(OutputFile);

			// Make sure the exit code reflects the failure state. XGE can sometimes fail transferring back.
			if(ExitCode == 0 && !NewWorker.bResult)
			{
				ExitCode = -1;
			}

			// If it's a hard failure, print the compile log to the regular log
			if(ExitCode == -1)
			{
				Writer.WriteLine("Failed to compile {0}, exit code {1}:", UniqueName, ExitCode);
				foreach(string Line in NewWorker.CompileLog)
				{
					Writer.WriteLine("    > {0}", Line.Replace("error:", "err:"));
				}
			}

			// Annotate the log data if this is from a failed attempt. It still may be useful for debugging purposes.
			if(ExitCode != 0)
			{
				NewWorker.SummaryLog.Insert(0, String.Format("ExitCode={0}", ExitCode));
				NewWorker.SummaryLog = NewWorker.SummaryLog.Select(x => "FAIL > " + x).ToList();

				NewWorker.CompileLog.Insert(0, String.Format("ExitCode={0}", ExitCode));
				NewWorker.CompileLog = NewWorker.CompileLog.Select(x => "FAIL > " + x).ToList();
			}

			// Append the log data back to the local output
			File.AppendAllLines(SummaryLogFile.FullName, NewWorker.SummaryLog);
			NewWorker.SummaryLog.Clear();

			File.AppendAllLines(CompileLogFile.FullName, NewWorker.CompileLog);
			NewWorker.CompileLog.Clear();

			// If we failed, return the 
			if(ExitCode != 0)
			{
				if(ExitCode == -1)
				{
					Writer.WriteLine("Warning: Failed to compile {0}, exit code {1}. Aborting.", UniqueName, ExitCode);
					return SequenceProbeResult.Failed;
				}
				else
				{
					Writer.WriteLine("Warning: Failed to compile {0}; exit code {1}. Will retry.", UniqueName, ExitCode);
					return SequenceProbeResult.FailedAllowRetry;
				}
			}

			// Update the task
			Worker = NewWorker;

			// Check if this is just an incremental update
			if(Type == SequenceProbeType.Verify)
			{
				// Save the task
				Worker.Serialize(TaskStateFile.FullName);

				// Return that we're done
				return SequenceProbeResult.Completed;
			}
			else if(Type == SequenceProbeType.Optimize)
			{
				if(Worker.RemainingFragmentCount > 0)
				{
					// Get the top-most fragment - the one we've just established is a dependency for this leaf node - and add it to the list of known dependencies
					SourceFragment NextFragment = Fragments[Worker.RemainingFragmentCount - 1];
					AddDependency(Worker, Fragments, Worker.RemainingFragmentCount - 1);
					Worker.SummaryLog.Add(String.Format("         [Added {0}: {1}]", Worker.RemainingFragmentCount - 1, Fragments[Worker.RemainingFragmentCount - 1].Location));

					// Save the task
					Worker.Serialize(TaskStateFile.FullName);

					// Otherwise, return that we've just updated
					return SequenceProbeResult.Updated;
				}
				else
				{
					// Save the task
					Worker.Serialize(TaskStateFile.FullName);

					// Return that we're done
					SetCompletedDependencies();
					return SequenceProbeResult.Completed;
				}
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Updates the dependencies on the target fragment if we're done
		/// </summary>
		void SetCompletedDependencies()
		{
			HashSet<SourceFragment> Dependencies = new HashSet<SourceFragment>();
			foreach(int DependencyIdx in Worker.KnownDependencies)
			{
				Dependencies.Add(Fragments[DependencyIdx]);
			}
			LastFragment.Dependencies = Dependencies;
		}

		/// <summary>
		/// Wait for any of the given tasks to complete
		/// </summary>
		/// <param name="Tasks"></param>
		/// <returns>Index of the completed task</returns>
		public static int WaitForAny(SequenceProbe[] Workers, int TimeoutMilliseconds)
		{
			return ManagedTaskProcess.WaitForAny(Workers.Select(x => x.ActiveInstance).ToArray(), TimeoutMilliseconds);
		}
	}
}
