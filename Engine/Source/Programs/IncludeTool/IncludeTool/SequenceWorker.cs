// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading.Tasks;
using IncludeTool.Support;
using System.Threading;

namespace IncludeTool
{
	/// <summary>
	/// Encapsulates the state of a sequence of fragments being optimized
	/// </summary>
	class SequenceWorker
	{
		/// <summary>
		/// Path to the temporary file to write out permutations of the source file to try compiling
		/// </summary>
		public string PermutationFileName;

		/// <summary>
		/// Path to the response file passed to the compiler
		/// </summary>
		public string ResponseFileName;

		/// <summary>
		/// Digest of the response file; for determining validity of the task data
		/// </summary>
		public string ResponseFileDigest;

		/// <summary>
		/// Array of filenames that we're optimizing. Used for display of initial state and output state; the actual file contents being optimized is stored in Lines.
		/// </summary>
		public string[] FragmentFileNames;

		/// <summary>
		/// Digest for each fragment
		/// </summary>
		public string[] FragmentDigests;

		/// <summary>
		/// Array of lines in the source file to analyze. The accompanying integer indicates the 
		/// </summary>
		public Tuple<int, string>[] Lines;

		/// <summary>
		/// Path to the compiler executable
		/// </summary>
		public string CompilerExe;

		/// <summary>
		/// The total number of fragments
		/// </summary>
		public int FragmentCount;

		/// <summary>
		/// The number of fragments from the start of the array which have not yet been checked, so are assumed to be required dependencies (in addition to the Dependencies set).
		/// </summary>
		public int RemainingFragmentCount;

		/// <summary>
		/// Set of indices into the array of fragments of known dependencies.
		/// </summary>
		public SortedSet<int> KnownDependencies = new SortedSet<int>();

		/// <summary>
		/// The number of compiles that have been performed
		/// </summary>
		public int CompileCount;

		/// <summary>
		/// Output lines for the summary log file
		/// </summary>
		public List<string> SummaryLog = new List<string>();

		/// <summary>
		/// Output lines for the compiler log file
		/// </summary>
		public List<string> CompileLog = new List<string>();

		/// <summary>
		/// 
		/// </summary>
		public bool bSingleStepMode = false;

		/// <summary>
		/// 
		/// </summary>
		public bool bResult = true;

		/// <summary>
		/// Private constructor. Used when deserializing from disk.
		/// </summary>
		SequenceWorker()
		{
		}

		/// <summary>
		/// Construct a new worker instance
		/// </summary>
		/// <param name="PermutationFileName">The file to write permutations to while searching for required headers</param>
		/// <param name="ResponseFileName">Response file containing compiler arguments for the file being compiled</param>
		/// <param name="ResponseFileDigest">Digest of the response file</param>
		/// <param name="FragmentFileNames">List of fragment filenames</param>
		/// <param name="FragmentDigests">Digests for each fragment</param>
		/// <param name="Lines">Array of lines in the file to optimize. Indicies in this array correspond to fragments, each of which will be enabled and disabled when looking for depenendcies.</param>
		/// <param name="CompilerExe">Executable for the compiler</param>
		public SequenceWorker(string PermutationFileName, string ResponseFileName, string ResponseFileDigest, string[] FragmentFileNames, string[] FragmentDigests, Tuple<int, string>[] Lines, string CompilerExe)
		{
			// Save the initial parameters
			this.PermutationFileName = PermutationFileName;
			this.ResponseFileName = ResponseFileName;
			this.ResponseFileDigest = ResponseFileDigest;
			this.FragmentFileNames = FragmentFileNames;
			this.FragmentDigests = FragmentDigests;
			this.Lines = Lines;
			this.CompilerExe = CompilerExe;

			// Set the fragemnt counters
			FragmentCount = FragmentFileNames.Length;
			RemainingFragmentCount = FragmentFileNames.Length;

			// Initialize the dependency list to contain the top fragment
			KnownDependencies.Add(FragmentFileNames.Length - 1);

			// Dump a list of all the includes in the order
			Log(SummaryLog, "Include order:");
			for(int Idx = 0; Idx < this.FragmentFileNames.Length; Idx++)
			{
				Log(SummaryLog, "  {0,12}: {1}", Idx, FragmentFileNames[Idx]);
			}
			Log(SummaryLog, "");

			// Create the table headings
			StringBuilder Heading = new StringBuilder("         ");
			for(int Idx = 0; Idx < KnownDependencies.Last() + 10 - 1; Idx += 10)
			{
				Heading.AppendFormat("| {0,-8}", Idx);
			}
			Log(SummaryLog, Heading.ToString());
		}

		/// <summary>
		/// Adds a line to the given log
		/// </summary>
		/// <param name="Log">List of lines for a log</param>
		/// <param name="Format">Formatting specification for the line to add</param>
		/// <param name="Args">Arguments for the line</param>
		static void Log(List<string> LogLines, string Format, params object[] Args)
		{
			LogLines.Add(String.Format(Format, Args));
		}

		/// <summary>
		/// Serialize this worker to a file
		/// </summary>
		/// <param name="FileName">Path to the file containing the worker information</param>
		public void Serialize(string FileName)
		{
			using (FileStream Stream = new FileStream(FileName, FileMode.Create, FileAccess.Write, FileShare.Read))
			{
				using (BinaryWriter Writer = new BinaryWriter(Stream, Encoding.UTF8, true))
				{
					Writer.Write(PermutationFileName);
					Writer.Write(ResponseFileName);
					Writer.Write(ResponseFileDigest);
					Writer.Write(FragmentCount);
					Writer.Write(RemainingFragmentCount);
					Writer.Write(CompileCount);
					Writer.Write(CompilerExe);
					Writer.Write(bSingleStepMode);
					Writer.Write(bResult);

					Writer.Write(KnownDependencies.Count);
					foreach(int KnownDependency in KnownDependencies)
					{
						Writer.Write(KnownDependency);
					}

					Writer.Write(FragmentFileNames.Length);
					foreach (string FragmentFileName in FragmentFileNames)
					{
						Writer.Write(FragmentFileName);
					}

					Writer.Write(FragmentDigests.Length);
					foreach (string FragmentDigest in FragmentDigests)
					{
						Writer.Write(FragmentDigest);
					}

					Writer.Write(Lines.Length);
					foreach (Tuple<int, string> Line in Lines)
					{
						Writer.Write(Line.Item1);
						Writer.Write(Line.Item2);
					}

					Writer.Write(SummaryLog.Count);
					foreach(string Line in SummaryLog)
					{
						Writer.Write(Line);
					}

					Writer.Write(CompileLog.Count);
					foreach(string Line in CompileLog)
					{
						Writer.Write(Line);
					}
				}
			}
		}

		/// <summary>
		/// Deserialize a worker from a file
		/// </summary>
		/// <param name="FileName">The filename to deserialize from</param>
		/// <returns>The deserialized worker instance</returns>
		public static SequenceWorker Deserialize(string FileName)
		{
			SequenceWorker Worker = new SequenceWorker();
			using (FileStream Stream = new FileStream(FileName, FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				using (BinaryReader Reader = new BinaryReader(Stream, Encoding.UTF8, true))
				{
					Worker.PermutationFileName = Reader.ReadString();
					Worker.ResponseFileName = Reader.ReadString();
					Worker.ResponseFileDigest = Reader.ReadString();
					Worker.FragmentCount = Reader.ReadInt32();
					Worker.RemainingFragmentCount = Reader.ReadInt32();
					Worker.CompileCount = Reader.ReadInt32();
					Worker.CompilerExe = Reader.ReadString();
					Worker.bSingleStepMode = Reader.ReadBoolean();
					Worker.bResult = Reader.ReadBoolean();

					int NumKnownDependencies = Reader.ReadInt32();
					for(int Idx = 0; Idx < NumKnownDependencies; Idx++)
					{
						Worker.KnownDependencies.Add(Reader.ReadInt32());
					}

					Worker.FragmentFileNames = new string[Reader.ReadInt32()];
					for(int Idx = 0; Idx < Worker.FragmentFileNames.Length; Idx++)
					{
						Worker.FragmentFileNames[Idx] = Reader.ReadString();
					}

					Worker.FragmentDigests = new string[Reader.ReadInt32()];
					for(int Idx = 0; Idx < Worker.FragmentDigests.Length; Idx++)
					{
						Worker.FragmentDigests[Idx] = Reader.ReadString();
					}

					Worker.Lines = new Tuple<int, string>[Reader.ReadInt32()];
					for(int Idx = 0; Idx < Worker.Lines.Length; Idx++)
					{
						int Tag = Reader.ReadInt32();
						string Line = Reader.ReadString();
						Worker.Lines[Idx] = Tuple.Create(Tag, Line);
					}

					int NumSummaryLogLines = Reader.ReadInt32();
					for (int Idx = 0; Idx < NumSummaryLogLines; Idx++)
					{
						Worker.SummaryLog.Add(Reader.ReadString());
					}

					int NumCompileLogLines = Reader.ReadInt32();
					for (int Idx = 0; Idx < NumCompileLogLines; Idx++)
					{
						Worker.CompileLog.Add(Reader.ReadString());
					}
				}
			}
			return Worker;
		}

		/// <summary>
		/// Compiles the permutation in its current state
		/// </summary>
		/// <param name="Writer">Writer for diagnostic messages</param>
		public void Verify(LineBasedTextWriter Writer)
		{
			bResult = CompilePermutation(new List<int> { RemainingFragmentCount }, 0, 0, 0);
		}

		/// <summary>
		/// Find the next dependency
		/// </summary>
		/// <param name="Writer">Writer for diagnostic messages</param>
		public void FindNextDependency(LineBasedTextWriter Writer)
		{
			// Update the required count for all the dependencies that are explicitly listed
			while(KnownDependencies.Contains(RemainingFragmentCount - 1))
			{
				RemainingFragmentCount--;
			}

			// Build a list of all valid potential dependency counts. This range is inclusive, since all and none are both valid.
			List<int> PotentialDependencyCounts = new List<int>{ 0 };
			for(int Idx = 0; Idx < RemainingFragmentCount; Idx++)
			{
				if(!KnownDependencies.Contains(Idx))
				{
					PotentialDependencyCounts.Add(Idx + 1);
				}
			}

			// On the first run, make sure the probe compiles with everything enabled
			int MaxRequired = PotentialDependencyCounts.Count - 1;
			if(CompileCount == 0)
			{
				bResult = CompilePermutation(PotentialDependencyCounts, 0, MaxRequired, MaxRequired);
				if(!bResult)
				{
					return;
				}
			}

			// Iteratively try to eliminate as many headers as possible.
			for(int MinRequired = 0; MinRequired < MaxRequired; )
			{
				// Simple binary search; round to zero. Pivot is a count of PotentialDependencies, where MinRequired <= Pivot < MaxRequired always.
				int Pivot = (MinRequired + MaxRequired) / 2;

				// Allow overriding the pivot in single-step mode, to optimize for situations where there's a sequence of dependencies
				if(bSingleStepMode)
				{
					Pivot = MaxRequired - 1;
					bSingleStepMode = false;
				}

				// Compile it
				bResult = CompilePermutation(PotentialDependencyCounts, MinRequired, Pivot, MaxRequired);

				// Update the range for the next step
				if(bResult)
				{
					MaxRequired = Pivot;
				}
				else
				{
					MinRequired = Pivot + 1;
				}
			}

			// Make sure we always finish on a successful compile, just to prevent any errors propagating to the next iteration
			if(!bResult)
			{
				bResult = CompilePermutation(PotentialDependencyCounts, MaxRequired, MaxRequired, MaxRequired);
				if(!bResult)
				{
					return;
				}
			}

			// Switch to single-step mode if we picked the last viable dependency
			if(MaxRequired == PotentialDependencyCounts.Count - 1)
			{
				bSingleStepMode = true;
			}

			// Now that we've finished compiling, store off the new maximum number of required dependencies
			RemainingFragmentCount = PotentialDependencyCounts[MaxRequired];

			// Write out the final include order once we're done
			if(RemainingFragmentCount == 0)
			{
				Log(SummaryLog, "");
				Log(SummaryLog, "Final include order:");
				foreach(int Idx in KnownDependencies)
				{
					Log(SummaryLog, "{0,8}: {1}", Idx, FragmentFileNames[Idx]);
				}
			}
		}

		/// <summary>
		/// Try to compile a given permutation
		/// </summary>
		/// <param name="PotentialDependencyCounts">Search domain for valid fragment dependency counts (ie. list of possible number of fragment dependencies)</param>
		/// <param name="MinRequired">Index into the PotentialDependencyCounts list of the binary search lower bound (inclusive)</param>
		/// <param name="Pivot">Index into the PotentialDependencyCounts list currently being tested</param>
		/// <param name="MaxRequired">Index into the PotentialDependencyCounts list of the binary search upper bound (exclusive)</param>
		/// <returns>True if the permutation compiled successfully</returns>
		bool CompilePermutation(List<int> PotentialDependencyCounts, int MinRequired, int Pivot, int MaxRequired)
		{
			Debug.Assert(MinRequired <= Pivot && Pivot <= MaxRequired);

			// Build the permutation
			BitArray Permutation = CreatePermutationArray(FragmentFileNames.Length, KnownDependencies, PotentialDependencyCounts[Pivot]);

			// Write out the file for this permutation
			WritePermutation(PermutationFileName, Lines, Permutation);

			// Test this permutation
			Stopwatch CompileStopwatch = Stopwatch.StartNew();
			Log(CompileLog, "Compiling attempt {0}...", CompileCount);
			Log(CompileLog, "Compiler={0}", CompilerExe);
			Log(CompileLog, "Dependencies=({0})", String.Join(", ", KnownDependencies));
			Log(CompileLog, "Permutation=({0})", String.Join(", ", Enumerable.Range(0, Permutation.Length).Where(x => Permutation[x]).Select(x => x.ToString())));
			Log(CompileLog, "PotentialDepdendencyCounts=({0})", String.Join(", ", PotentialDependencyCounts));
			Log(CompileLog, "MinRequired={0}, Pivot={1}, MaxRequired={2}", MinRequired, Pivot, MaxRequired);
			bool bResult = Compile(CompileLog);
			Log(CompileLog, "");

			// Prepare the next table row
			string Attempt = GetCurrentIterationRow(PotentialDependencyCounts, MinRequired, Pivot, MaxRequired, KnownDependencies, Permutation);
			Log(SummaryLog, "{0,4}/{1,-3} [{2}] {3,5} {4}", CompileCount, KnownDependencies.Count, Attempt, bResult, CompileStopwatch.Elapsed);

			// Increment the attempt counter
			CompileCount++;
			return bResult;
		}

		/// <summary>
		/// Invokes the compiler with the given arguments, capturing the output to an array of strings. Retries if the compiler fails with an XGE error.
		/// </summary>
		/// <param name="Arguments">Arguments for the compiler</param>
		/// <param name="Output">List of output lines for the compiler</param>
		/// <returns>True if the compiler succeeded</returns>
		bool Compile(List<string> Output)
		{
			for(int Idx = 0; Idx < 5; Idx++)
			{
				int LineIdx = Output.Count;
				if(CompileInternal(Output))
				{
					return true;
				}
				for(;;LineIdx++)
				{
					if(LineIdx == Output.Count)
					{
						return false;
					}
					else if(Output[LineIdx].Contains("IncrediBuild") && Output[LineIdx].Contains("Exception"))
					{
						break;
					}
				}
				Output.Add("Exception; retrying.");
			}
			Output.Add("Failed 5 times, aborting");
			return false;
		}

		const uint SEM_FAILCRITICALERRORS  = 0x01;
		const uint SEM_NOGPFAULTERRORBOX = 0x0002;

		[DllImport("kernel32.dll")]
		static extern uint SetErrorMode(uint Mode);

		/// <summary>
		/// Invokes the compiler with the given arguments, capturing the output to an array of strings
		/// </summary>
		/// <param name="Output">List of output lines for the compiler</param>
		/// <returns>True if the compiler succeeded</returns>
		bool CompileInternal(List<string> Output)
		{
			// Disable the WER dialog in case the compiler crashes (this is inherited by child processes).
			SetErrorMode(SetErrorMode(0) | SEM_NOGPFAULTERRORBOX);

			// Increase the timeout with each attempt
			using (ManagedProcess NewProcess = new ManagedProcess(CompilerExe, String.Format("@\"{0}\"", ResponseFileName), null, null, null, ManagedProcessPriority.BelowNormal, 20 * 1024 * 1024 * 1024UL))
			{
				const int TimeoutMinutes = 10;
				if(NewProcess.TryReadAllLines(TimeSpan.FromMinutes(TimeoutMinutes), Output))
				{
					return NewProcess.ExitCode == 0;
				}
				Console.WriteLine("Compile timeout after {0} minutes", TimeoutMinutes);
				NewProcess.Terminate(1);
			}

			// Sometimes see errors due to terminated process still having a file open. Give it a chance to clean up.
			Thread.Sleep(30 * 1000);
			return false;
		}

		/// <summary>
		/// Create a permutation array for the known dependencies and given pivot
		/// </summary>
		/// <param name="Length">The number of entries in the permutation array</param>
		/// <param name="Dependencies">Set of known dependencies</param>
		/// <param name="PivotCount">The current pivot</param>
		/// <returns>BitArray corresponding to the given permutation, indicating which fragments should be included</returns>
		static BitArray CreatePermutationArray(int Length, IEnumerable<int> Dependencies, int PivotCount)
		{
			// Check that the pivot is zero or positive. It is a count of headers, not an index to the last header.
			Debug.Assert(PivotCount >= 0);

			// Build an bitmap of all the headers required in this permutation
			BitArray Permutation = new BitArray(Length);

			// Add all the dependencies up to the pivot
			for(int DependencyIdx = 0; DependencyIdx < PivotCount; DependencyIdx++)
			{
				Permutation[DependencyIdx] = true;
			}

			// Add all the known dependencies
			foreach(int DependencyIdx in Dependencies)
			{
				Permutation[DependencyIdx] = true;
			}

			// Return the array
			return Permutation;
		}

		/// <summary>
		/// Writes a file containing the given permutation
		/// </summary>
		/// <param name="PermutationFileName">The source file to write to</param>
		/// <param name="Lines">The array of lines to be enabled or disabled based on the permutation</param>
		/// <param name="Permutation">The permutation of lines to write</param>
		/// <returns>The chosen precompiled header</returns>
		static void WritePermutation(string PermutationFileName, Tuple<int, string>[] Lines, BitArray Permutation)
		{
			StreamWriter PermutationWriter = null;
			try
			{
				// Keep retrying to open the file for writing. Previous compiler instances may take a while to completely close.
				for(;;)
				{
					try
					{
						PermutationWriter = new StreamWriter(PermutationFileName);
						break;
					}
					catch(IOException)
					{
						Console.WriteLine("Couldn't open {0}; waiting 10s and retrying...", PermutationFileName);
						Thread.Sleep(10 * 1000);
					}
				}

				// Write the contents
				int LineIdx = 0;
				for(; LineIdx < Lines.Length; LineIdx++)
				{
					Tuple<int, string> Line = Lines[LineIdx];
					if(Line.Item1 < 0 || Permutation[Line.Item1])
					{
						PermutationWriter.WriteLine("{0,-12}    {1}", String.Format("/* {0} */", Line.Item1), Line.Item2);
					}
					else
					{
						PermutationWriter.WriteLine("{0,-12} // {1}", String.Format("/* {0} */", Line.Item1), Line.Item2);
					}
				}
			}
			finally
			{
				if(PermutationWriter != null)
				{
					PermutationWriter.Dispose();
				}
			}
		}

		/// <summary>
		/// Formats a string containing an ASCII chart representing the permutation being compiled. Key is as follows:
		///   'x' Known dependency  
		///   ' ' Known non-dependency
		///   'o' Outside current search range, treating as a required dependency
		///   '-' Unknown dependency, currently testing as enabled
		///   '.' Unknown dependency, currently testing as disabled
		/// </summary>
		/// <param name="PotentialDependencyCounts">Search domain for valid fragment dependency counts (ie. list of possible number of fragment dependencies)</param>
		/// <param name="MinRequired">Index into the PotentialDependencyCounts list of the binary search lower bound (inclusive)</param>
		/// <param name="Pivot">Index into the PotentialDependencyCounts list currently being tested</param>
		/// <param name="MaxRequired">Index into the PotentialDependencyCounts list of the binary search upper bound (exclusive)</param>
		/// <param name="Dependencies">Indices of known fragment dependencies</param>
		/// <param name="Permutation">Current bitmask for the current permutation of fragments</param>
		/// <returns>String representing the current iteration</returns>
		static string GetCurrentIterationRow(List<int> PotentialDependencyCounts, int MinRequired, int Pivot, int MaxRequired, SortedSet<int> Dependencies, BitArray Permutation)
		{
			char[] Characters = new char[Dependencies.Last() + 1];
			for(int Idx = 0; Idx < PotentialDependencyCounts[MinRequired]; Idx++)
			{
				Characters[Idx] = 'o';
			}
			for(int Idx = PotentialDependencyCounts[MinRequired]; Idx < PotentialDependencyCounts[Pivot]; Idx++)
			{
				Characters[Idx] = '-';
			}
			for(int Idx = PotentialDependencyCounts[Pivot]; Idx < PotentialDependencyCounts[MaxRequired]; Idx++)
			{
				Characters[Idx] = '.';
			}
			for(int Idx = PotentialDependencyCounts[MaxRequired]; Idx < Characters.Length; Idx++)
			{
				Characters[Idx] = ' ';
			}
			foreach(int Idx in Dependencies)
			{
				Characters[Idx] = 'x';
			}
			for(int Idx = 0; Idx < Permutation.Count; Idx++)
			{
				if(!Permutation[Idx])
				{
					Characters[Idx] = ' ';
				}
			}
			return new string(Characters);
		}
	}
}
