// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Flags which apply to a source file
	/// </summary>
	[Flags]
	enum SourceFileFlags
	{
		/// <summary>
		/// This file is external, and cannot be optimized by the tool
		/// </summary>
		External = 0x01,

		/// <summary>
		/// This file is a .generated.h file
		/// </summary>
		GeneratedHeader = 0x02,

		/// <summary>
		/// This file is a generated classes.h file
		/// </summary>
		GeneratedClassesHeader = 0x04,

		/// <summary>
		/// Includes of this file are context-specific to the location that they are included from, and are forced to be inlined into a fragment rather 
		/// than being treated as an optional dependency. Typically for headers which rely on mutating the processor environment which can occur inside 
		/// other declarations (eg. ScriptSerialization.h). NOTE: These files will be included from their original location, and must NOT include other files.
		/// </summary>
		Inline = 0x08,

		/// <summary>
		/// Includes of this file must be kept in the files that currently include them. Platform-specific headers are typically marked like this.
		/// Unlike the inline flag, pinned files may contain multiple fragments and includes of other files which need optimizing, but they should 
		/// not be included outside their current location. Pinned files are given a node in the sequence tree, and are forced to be included.
		/// </summary>
		Pinned = 0x10,

		/// <summary>
		/// This file is meant to compile in isolation
		/// </summary>
		Standalone = 0x20,

		/// <summary>
		/// This file should not be optimized, and is meant to retain all its original includes (eg. Core.h)
		/// </summary>
		Aggregate = 0x80,

		/// <summary>
		/// This file should be optimized
		/// </summary>
		Optimize = 0x100,

		/// <summary>
		/// This is a monolithic header
		/// </summary>
		Monolithic = 0x200,

		/// <summary>
		/// This file is a translation unit
		/// </summary>
		TranslationUnit = 0x400,

		/// <summary>
		/// Whether this file is allowed to have multiple fragments
		/// </summary>
		AllowMultipleFragments = 0x800,

		/// <summary>
		/// Special flag to mark the CoreTypes.h header, so it can be included as a monolithic header.
		/// </summary>
		IsCoreTypes = 0x1000,

		/// <summary>
		/// Special flag to mark the CoreMinimal.h header, so it can be included as a monolithic header.
		/// </summary>
		IsCoreMinimal = 0x2000,

		/// <summary>
		/// This file is in a public directory (eg. ModuleName/Public or ModuleName/Classes).
		/// </summary>
		Public = 0x4000,

		/// <summary>
		/// This header consists of forward declarations
		/// </summary>
		FwdHeader = 0x8000,

		/// <summary>
		/// This file should be output
		/// </summary>
		Output = 0x10000,

		/// <summary>
		/// Do not scan this file for exported symbols
		/// </summary>
		IgnoreExportedSymbols = 0x20000,
	}

	/// <summary>
	/// Encapsulates metadata for a C++ source file
	/// </summary>
	[Serializable]
	class SourceFile
	{
		/// <summary>
		/// The location of the file on disk
		/// </summary>
		public readonly FileReference Location;

		/// <summary>
		/// The contents of the file. May be null for opaque files.
		/// </summary>
		public readonly TextBuffer Text;

		/// <summary>
		/// Markup dividing up this source file
		/// </summary>
		public readonly PreprocessorMarkup[] Markup;

		/// <summary>
		/// Index into the Markup array of the first item after any header guards and/or #pragma once directives
		/// </summary>
		public readonly int BodyMinIdx;

		/// <summary>
		/// Index into the Markup array one past the last item not part of a header guard
		/// </summary>
		public readonly int BodyMaxIdx;

		/// <summary>
		/// The module that this source file belongs to.
		/// </summary>
		public BuildModule Module;

		/// <summary>
		/// Properties of this file
		/// </summary>
		public SourceFileFlags Flags;

		/// <summary>
		/// Parsed fragments in this file. Only set for files which are being optimized.
		/// </summary>
		public SourceFragment[] Fragments;

		/// <summary>
		/// For files that occur in matched pairs (eg. PreWindowsApi.h, PostWindowsApi.h), references the counterpart to this file. Otherwise null.
		/// </summary>
		public SourceFile Counterpart
		{
			get;
			private set;
		}

		/// <summary>
		/// Files inserted at the start of this file to make it possible to include this file on its own
		/// </summary>
		public List<SourceFile> MissingIncludes = new List<SourceFile>();

		/// <summary>
		/// Forward declarations which are inserted prior to optimization
		/// </summary>
		public HashList<string> ForwardDeclarations = new HashList<string>();

		/// <summary>
		/// Verbose per-file log lines
		/// </summary>
		public List<string> VerboseOutput = new List<string>();

		/// <summary>
		/// Regex for additional options
		/// </summary>
		static Regex OptionsPattern = new Regex(@"^\s*//\s*\[\[\s*IncludeTool\s*:\s*([a-zA-Z, \t]+)\]\]", RegexOptions.IgnoreCase);

		/// <summary>
		/// Construct a SourceFile from the given arguments
		/// </summary>
		/// <param name="Location">Location of the file</param>
		/// <param name="Text">Contents of the file</param>
		/// <param name="Flags">Properties of the file</param>
		public SourceFile(FileReference Location, TextBuffer Text, SourceFileFlags Flags)
		{
			// Check for directives specifying additional flags for this file in the source text
			if (Text != null)
			{
				foreach (string Line in Text.Lines)
				{
					Match Match = OptionsPattern.Match(Line);
					if (Match.Success)
					{
						foreach (string FlagText in Match.Groups[1].Value.Split(',').Select(x => x.Trim()).Where(x => x.Length > 0))
						{
							SourceFileFlags Flag;
							if (Enum.TryParse(FlagText, true, out Flag))
							{
								Flags |= Flag;
							}
							else
							{
								throw new Exception(String.Format("{0}: Invalid source file flag '{1}'", Location, FlagText));
							}
						}
					}
				}
			}

			// Inline files cannot be standalone
			if((Flags & SourceFileFlags.Inline) != 0)
			{
				Flags &= ~SourceFileFlags.Standalone;
			}

			// Save the parameters
			this.Location = Location;
			this.Text = Text;
			this.Flags = Flags;

			// Read the preprocsesor markup
			if(Text == null)
			{
				Markup = new PreprocessorMarkup[0];
			}
			else
			{
				Markup = PreprocessorMarkup.ParseArray(Location?.FullName ?? "unknown", Text);
			}

			// Find the markup range which excludes header guards
			BodyMinIdx = 0;
			BodyMaxIdx = Markup.Length;
			while(BodyMaxIdx > BodyMinIdx)
			{
				if(!SkipPragmaOnce(Markup, ref BodyMinIdx) && !SkipHeaderGuard(Markup, ref BodyMinIdx, ref BodyMaxIdx))
				{
					break;
				}
			}

			// Inline files must not have a header guard (because it would result in a different derivation), and must not include any other files (because we include them
			// from their original location)
			if((Flags & SourceFileFlags.Inline) != 0)
			{
				if(HasHeaderGuard)
				{
					throw new Exception("Files marked as 'inline' may not have a header guard, since they will be included directly.");
				}
			}
		}

		/// <summary>
		/// Appends to the per-file log
		/// </summary>
		/// <param name="Format">Format string</param>
		/// <param name="Args">Format arguments</param>
		public void LogVerbose(string Format, params object[] Args)
		{
			VerboseOutput.Add(String.Format(Format, Args));
		}

		/// <summary>
		/// Sets the counterpart on a file, and checks that it meets the requirements for doing so
		/// </summary>
		/// <param name="File"></param>
		/// <param name="CounterpartFile"></param>
		public void SetCounterpart(SourceFile OtherFile)
		{
			if(Counterpart != OtherFile)
			{
				if(Counterpart != null)
				{
					throw new Exception("File already has a counterpart set");
				}

				Counterpart = OtherFile;

				if(HasHeaderGuard)
				{
					throw new Exception("Files with counterparts may not have a header guard, since they will be included directly.");
				}

				OtherFile.SetCounterpart(this);
			}
		}

		/// <summary>
		/// Checks whether the markup matches a #pragma once directive
		/// </summary>
		/// <param name="Markup">The parsed markup for this file</param>
		/// <param name="BodyMinIdx">The current minimum index of non-header guard markup in this file</param>
		/// <returns>True if the markup represents a #pragma once directive</returns>
		static bool SkipPragmaOnce(PreprocessorMarkup[] Markup, ref int BodyMinIdx)
		{
			PreprocessorMarkup FirstMarkup = Markup[BodyMinIdx];
			if(FirstMarkup.Type == PreprocessorMarkupType.Pragma && FirstMarkup.Tokens.Count == 1 && FirstMarkup.Tokens[0].Text == "once")
			{
				BodyMinIdx++;
				return true;
			}
			return false;
		}

		/// <summary>
		/// Checks whether the markup matches a C-style header guard
		/// </summary>
		/// <param name="Markup">The parsed markup for this file</param>
		/// <param name="BodyMinIdx">The current minimum index of non-header guard markup in this file</param>
		/// <param name="BodyMinIdx">The current maximum index of non-header guard markup in this file</param>
		/// <returns>True if the text has a header guard</returns>
		static bool SkipHeaderGuard(PreprocessorMarkup[] Markup, ref int BodyMinIdx, ref int BodyMaxIdx)
		{
			// Make sure there are enough markup entries in this list
			if(BodyMinIdx + 2 < BodyMaxIdx)
			{
				// Get the define used for the header guard
				string DefineToken;
				if(Markup[BodyMinIdx].Type == PreprocessorMarkupType.Ifndef && Markup[BodyMinIdx].Tokens.Count == 1 && Markup[BodyMinIdx].Tokens[0].Type == TokenType.Identifier)
				{
					DefineToken = Markup[BodyMinIdx].Tokens[0].Text;
				}
				else if(Markup[BodyMinIdx].Type == PreprocessorMarkupType.If && Markup[BodyMinIdx].Tokens.Count == 3 && Markup[BodyMinIdx].Tokens[0].Text == "!" && Markup[BodyMinIdx].Tokens[1].Text == "defined" && Markup[BodyMinIdx].Tokens[2].Type == TokenType.Identifier)
				{
					DefineToken = Markup[BodyMinIdx].Tokens[2].Text;
				}
				else
				{
					DefineToken = null;
				}

				// Check it starts with an #if or #ifdef
				if (Markup[BodyMinIdx + 1].Type == PreprocessorMarkupType.Define && Markup[BodyMinIdx + 1].Tokens.Count == 1 && Markup[BodyMinIdx + 1].Tokens[0].Text == DefineToken)
				{
					// Find the point at which the include depth goes back to zero
					int EndIdx = BodyMinIdx + 1;
					for (int Depth = 1; EndIdx < BodyMaxIdx; EndIdx++)
					{
						Depth += Markup[EndIdx].GetConditionDepthDelta();
						if (Depth == 0)
						{
							break;
						}
					}

					// Check it matched the end of the file
					if (EndIdx == BodyMaxIdx - 1)
					{
						BodyMinIdx += 2;
						BodyMaxIdx -= 1;
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Build a flat list of fragments included by this file
		/// </summary>
		/// <returns>List of included fragments, in order</returns>
		public List<SourceFragment> FindIncludedFragments()
		{
			List<SourceFragment> IncludedFragments = new List<SourceFragment>();
			FindIncludedFragments(IncludedFragments, null, new HashSet<SourceFile>());
			return IncludedFragments;
		}

		/// <summary>
		/// Build a flat list of included fragments
		/// </summary>
		/// <param name="IncludedFragments">The flat list of fragments</param>
		/// <param name="IncludeStackChanges">Optional list to store changes to the include stack, and the count of fragments at that point. Records with a file set to null indicate that the end of the topmost file.</param>
		/// <param name="UniqueIncludedFiles">Set of files which have already been visited. Used to resolve files with header guards.</param>
		public void FindIncludedFragments(List<SourceFragment> IncludedFragments, List<Tuple<int, SourceFile>> IncludeStackChanges, HashSet<SourceFile> UniqueIncludedFiles)
		{
			if((Flags & SourceFileFlags.Inline) == 0)
			{
				if(IncludeStackChanges != null)
				{
					IncludeStackChanges.Add(new Tuple<int, SourceFile>(IncludedFragments.Count, this));
				}

				foreach(SourceFragment Fragment in Fragments)
				{
					Fragment.FindIncludedFragments(IncludedFragments, IncludeStackChanges, UniqueIncludedFiles);
				}

				if(IncludeStackChanges != null)
				{
					IncludeStackChanges.Add(new Tuple<int, SourceFile>(IncludedFragments.Count, null));
				}
			}
		}

		/// <summary>
		/// Whether the file has a header guard that prevents it being included more than once in a single translation unit
		/// </summary>
		public bool HasHeaderGuard
		{
			get { return BodyMinIdx > 0 || BodyMaxIdx < Markup.Length; }
		}

		/// <summary>
		/// Write out an optimized file to the given location
		/// </summary>
		/// <param name="IncludePaths">Base directories for relative include paths</param>
		/// <param name="SystemIncludePaths">Base directories for system include paths</param>
		/// <param name="Writer">Writer for the output text</param>
		public void Write(IEnumerable<DirectoryReference> IncludePaths, IEnumerable<DirectoryReference> SystemIncludePaths, TextWriter Writer, bool bRemoveForwardDeclarations, LineBasedTextWriter Log)
		{
			// Write the file header
			TextLocation LastLocation = Text.Start;

			// Write the standalone includes
			if(MissingIncludes.Count > 0)
			{
				TextLocation BoilerplateLocation = (BodyMinIdx == Markup.Length || (Markup[BodyMinIdx].Type != PreprocessorMarkupType.Include && BodyMinIdx > 0))? Markup[BodyMinIdx - 1].EndLocation : Markup[BodyMinIdx].Location;
				WriteLines(Text, LastLocation, BoilerplateLocation, Writer);
				LastLocation = BoilerplateLocation;

				if (LastLocation.LineIdx > 0 && Text.Lines[LastLocation.LineIdx - 1].TrimEnd().Length > 0)
				{
					if(LastLocation.LineIdx + 1 < Text.Lines.Length && Text.Lines[LastLocation.LineIdx].TrimEnd().Length == 0 && Text.Lines[LastLocation.LineIdx + 1].TrimEnd().Length == 0)
					{
						LastLocation.LineIdx++;
					}
					Writer.WriteLine();
				}
				foreach(SourceFile MissingInclude in MissingIncludes)
				{
					string IncludeText = FormatInclude(Location.Directory, MissingInclude.Location, IncludePaths, SystemIncludePaths, Log);
					Writer.WriteLine("#include {0}", IncludeText);
				}
			}

			// Figure out before which markup object to write forward declarations, skipping over all the includes at the start of the file
			int ForwardDeclarationsBeforeMarkupIdx = -1;
			if((Flags & SourceFileFlags.TranslationUnit) == 0)
			{
				int ConditionDepth = 0;
				for(int MarkupIdx = BodyMinIdx; MarkupIdx < Markup.Length; MarkupIdx++)
				{
					if(ConditionDepth == 0)
					{
						ForwardDeclarationsBeforeMarkupIdx = MarkupIdx;
					}
					if(Markup[MarkupIdx].Type == PreprocessorMarkupType.Text)
					{
						break;
					}
					ConditionDepth += Markup[MarkupIdx].GetConditionDepthDelta();
				}
			}

			// Write all the other markup
			for(int MarkupIdx = BodyMinIdx; MarkupIdx < Markup.Length; MarkupIdx++)
			{
				PreprocessorMarkup ThisMarkup = Markup[MarkupIdx];

				// Write the forward declarations
				if(MarkupIdx == ForwardDeclarationsBeforeMarkupIdx)
				{
					// Write out at least up to the end of the last markup
					if(MarkupIdx > 0 && LastLocation <= Markup[MarkupIdx - 1].EndLocation)
					{
						WriteLines(Text, LastLocation, Markup[MarkupIdx - 1].EndLocation, Writer);
						LastLocation = Markup[MarkupIdx - 1].EndLocation;
					}

					// Skip a blank line in the existing text.
					TextLocation NewLastLocation = LastLocation;
					if(LastLocation.LineIdx < Text.Lines.Length && String.IsNullOrWhiteSpace(Text.Lines[LastLocation.LineIdx]))
					{
						NewLastLocation = new TextLocation(LastLocation.LineIdx + 1, 0);
					}

					// Merge all the existing forward declarations with the new set.
					HashSet<string> PreviousForwardDeclarations = new HashSet<string>();
					while(NewLastLocation.LineIdx < Text.Lines.Length)
					{
						string TrimLine = Text.Lines[NewLastLocation.LineIdx].Trim();
						if(TrimLine.Length > 0 && !TrimLine.Equals("// Forward declarations", StringComparison.OrdinalIgnoreCase) && !TrimLine.Equals("// Forward declarations.", StringComparison.OrdinalIgnoreCase))
						{
							// Create a token reader for the current line
							TokenReader Reader = new TokenReader(Location.FullName, Text, new TextLocation(NewLastLocation.LineIdx, 0), new TextLocation(NewLastLocation.LineIdx, Text.Lines[NewLastLocation.LineIdx].Length));

							// Read it into a buffer
							List<Token> Tokens = new List<Token>();
							while (Reader.MoveNext())
							{
								Tokens.Add(Reader.Current);
							}

							// Check it matches the syntax for a forward declaration, and add it to the list if it does
							if(Tokens.Count == 3 && (Tokens[0].Text == "struct" || Tokens[0].Text == "class") && Tokens[1].Type == TokenType.Identifier && Tokens[2].Text == ";")
							{
								PreviousForwardDeclarations.Add(String.Format("{0} {1};", Tokens[0].Text, Tokens[1].Text));
							}
							else if(Tokens.Count == 4 && Tokens[0].Text == "enum" && Tokens[1].Text == "class" && Tokens[2].Type == TokenType.Identifier && Tokens[3].Text == ";")
							{
								PreviousForwardDeclarations.Add(String.Format("enum class {0};", Tokens[2].Text));
							}
							else if(Tokens.Count == 6 && Tokens[0].Text == "enum" && Tokens[1].Text == "class" && Tokens[2].Type == TokenType.Identifier && Tokens[3].Text == ":" && Tokens[4].Type == TokenType.Identifier && Tokens[5].Text == ";")
							{
								PreviousForwardDeclarations.Add(String.Format("enum class {0} : {1};", Tokens[2].Text, Tokens[4].Text));
							}
							else if(ForwardDeclarations.Contains(Text.Lines[NewLastLocation.LineIdx]))
							{
								PreviousForwardDeclarations.Add(Text.Lines[NewLastLocation.LineIdx]);
							}
							else
							{
								break;
							}
						}
						NewLastLocation = new TextLocation(NewLastLocation.LineIdx + 1, 0);
					}

					// Create a full list of new forward declarations, combining with the ones that are already there. Normally we optimize with the forward declarations present,
					// so we shouldn't remove any unless running a specific pass designed to do so.
					HashSet<string> MergedForwardDeclarations = new HashSet<string>(ForwardDeclarations);
					if(!bRemoveForwardDeclarations)
					{
						MergedForwardDeclarations.UnionWith(PreviousForwardDeclarations);
					}

					// Write them out
					if(MergedForwardDeclarations.Count > 0)
					{
						Writer.WriteLine();
						foreach (string ForwardDeclaration in MergedForwardDeclarations.Distinct().OrderBy(x => GetForwardDeclarationSortKey(x)).ThenBy(x => x))
						{
							Writer.WriteLine("{0}{1}", GetIndent(MarkupIdx), ForwardDeclaration);
						}
						Writer.WriteLine();
						LastLocation = NewLastLocation;
					}
					else if(PreviousForwardDeclarations.Count > 0)
					{
						Writer.WriteLine();
						LastLocation = NewLastLocation;
					}
				}

				// Write the includes
				if(ThisMarkup.Type == PreprocessorMarkupType.Include && ThisMarkup.IsActive && !ThisMarkup.IsInlineInclude())
				{
					// Write up to the start of this include
					WriteLines(Text, LastLocation, ThisMarkup.Location, Writer);

					// Get the original include text. Some modules - particularly editor modules - include headers from other modules based from Engine/Source which are not listed as dependencies. If
					// the original include is based from a shallower directory than the one we would include otherwise, we'll use that instead.
					string OriginalIncludeText = null;
					if(ThisMarkup.Tokens.Count == 1)
					{
						OriginalIncludeText = ThisMarkup.Tokens[0].Text.Replace('\\', '/');
					}

					// Write the replacement includes
					foreach(SourceFile OutputIncludedFile in ThisMarkup.OutputIncludedFiles)
					{
						string IncludeText = FormatInclude(Location.Directory, OutputIncludedFile.Location, IncludePaths, SystemIncludePaths, Log);
						if(OutputIncludedFile == ThisMarkup.IncludedFile && Rules.IsExternalIncludeMacro(ThisMarkup.Tokens))
						{
							IncludeText = Token.Format(ThisMarkup.Tokens);
						}
						else if(OutputIncludedFile == ThisMarkup.IncludedFile && (OutputIncludedFile.Flags & SourceFileFlags.External) != 0)
						{
							IncludeText = OriginalIncludeText;
						}
						else if(OriginalIncludeText != null && (Flags & SourceFileFlags.TranslationUnit) == 0 && OriginalIncludeText.EndsWith(IncludeText.TrimStart('\"'), StringComparison.OrdinalIgnoreCase) && (OriginalIncludeText.StartsWith("\"Runtime/", StringComparison.InvariantCultureIgnoreCase) || OriginalIncludeText.StartsWith("\"Developer/", StringComparison.InvariantCultureIgnoreCase) || OriginalIncludeText.StartsWith("\"Editor/", StringComparison.InvariantCultureIgnoreCase)))
						{
							IncludeText = OriginalIncludeText;
						}
						Writer.WriteLine("{0}#include {1}", GetIndent(MarkupIdx), IncludeText);
					}

					// Copy any formatting
					if(ThisMarkup.EndLocation.LineIdx > ThisMarkup.Location.LineIdx + 1)
					{
						WriteLines(Text, new TextLocation(ThisMarkup.Location.LineIdx + 1, 0), ThisMarkup.EndLocation, Writer);
					}

					// Update the location to the start of the next line
					LastLocation = new TextLocation(ThisMarkup.Location.LineIdx + 1, 0);
				}
			}

			// Write to the end of the file
			WriteLines(Text, LastLocation, Text.End, Writer);
		}

		/// <summary>
		/// Returns a sort key for the type of forward declaration. Used to ensure a stable output.
		/// </summary>
		/// <param name="ForwardDeclaration">The text of the forward declaration</param>
		/// <returns>Integer indicating the order in which to sort the forward declaration</returns>
		int GetForwardDeclarationSortKey(string ForwardDeclaration)
		{
			if(ForwardDeclaration.StartsWith("class"))
			{
				return 1;
			}
			else if(ForwardDeclaration.StartsWith("struct"))
			{
				return 2;
			}
			else if(ForwardDeclaration.StartsWith("enum"))
			{
				return 3;
			}
			else
			{
				return 4;
			}
		}

		/// <summary>
		/// Returns the whitespace indent for the markup at the given index
		/// </summary>
		/// <param name="MarkupIdx">Index of the markup to retrieve whitespace for</param>
		/// <returns>The whitespace before this markup</returns>
		string GetIndent(int MarkupIdx)
		{
			string Line = Text.Lines[Markup[MarkupIdx].Location.LineIdx];
			for(int Idx = 0; Idx < Line.Length; Idx++)
			{
				if(!Char.IsWhiteSpace(Line[Idx]))
				{
					return Line.Substring(0, Idx);
				}
			}
			return "";
		}

		/// <summary>
		/// Split the given line into tokens
		/// </summary>
		/// <param name="FileName">File being parsed</param>
		/// <param name="Text">Buffer to read from</param>
		/// <param name="LineIdx">Index of the line to tokenize</param>
		/// <returns>Sequence of tokens</returns>
		static IEnumerable<Token> TokenizeLine(string FileName, TextBuffer Text, int LineIdx)
		{
			TokenReader Reader = new TokenReader(FileName, Text, new TextLocation(LineIdx, 0), new TextLocation(LineIdx, Text.Lines[LineIdx].Length));
			while(Reader.MoveNext())
			{
				yield return Reader.Current;
			}
		}

		/// <summary>
		/// Tests whether the given token is an identifier
		/// </summary>
		/// <param name="Token">Token to check</param>
		/// <returns>True if </returns>
		static bool IsIdentifier(string Token)
		{
			if(Token.Length == 0)
			{
				return false;
			}
			if((Token[0] < 'a' || Token[0] > 'z') && (Token[0] < 'A' || Token[0] > 'Z') && Token[0] != '_')
			{
				return false;
			}
			for(int Idx = 0; Idx < Token.Length; Idx++)
			{
				if((Token[Idx] < 'a' || Token[Idx] > 'z') && (Token[Idx] < 'A' || Token[Idx] > 'Z') && (Token[Idx] < '0' || Token[Idx] > '9') && Token[Idx] != '_')
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Writes a blank line, if the previous line is not already a blank line
		/// </summary>
		/// <param name="Text">The text buffer to read from</param>
		/// <param name="LastLocation">The current output position</param>
		static void WriteBlankLine(TextBuffer Text, ref TextLocation LastLocation, TextWriter Writer)
		{
			if (LastLocation.LineIdx > 0 && Text.Lines[LastLocation.LineIdx - 1].TrimEnd().Length > 0)
			{
				if(LastLocation.LineIdx + 1 < Text.Lines.Length && Text.Lines[LastLocation.LineIdx].TrimEnd().Length == 0 && Text.Lines[LastLocation.LineIdx + 1].TrimEnd().Length == 0)
				{
					LastLocation.LineIdx++;
				}
				Writer.WriteLine();
			}
		}

		/// <summary>
		/// Copies a sequence of lines into an output writer
		/// </summary>
		/// <param name="Text">The input text</param>
		/// <param name="Location">Start location to copy</param>
		/// <param name="EndLocation">End location to copy</param>
		/// <param name="Writer">The output writer</param>
		static void WriteLines(TextBuffer Text, TextLocation Location, TextLocation EndLocation, TextWriter Writer)
		{
			string[] TextLines = Text.Extract(Location, EndLocation);
			for(int Idx = 0; Idx < TextLines.Length - 1; Idx++)
			{
				Writer.WriteLine(TextLines[Idx]);
			}
			Writer.Write(TextLines[TextLines.Length - 1]);
		}

		/// <summary>
		/// Format an include of the given file
		/// </summary>
		/// <param name="FromDirectory">The directory containing the file with the #include directive</param>
		/// <param name="IncludeFile">File to include</param>
		/// <param name="IncludePaths">Directories to base relative include paths from</param>
		/// <param name="SystemIncludePaths">Directories to base system include paths from</param>
		/// <returns>Formatted include path, with surrounding quotes</returns>
		public static string FormatInclude(DirectoryReference FromDirectory, FileReference IncludeFile, IEnumerable<DirectoryReference> IncludePaths, IEnumerable<DirectoryReference> SystemIncludePaths, LineBasedTextWriter Log)
		{
			string IncludeText;
			if(!TryFormatInclude(FromDirectory, IncludeFile, IncludePaths, SystemIncludePaths, out IncludeText))
			{
				Log.WriteLine("warning: cannot create relative path for {0}; assuming <{1}>", IncludeFile.FullName, IncludeFile.GetFileName());
				IncludeText = "<" + IncludeFile.GetFileName() + ">";
			}
			return IncludeText;
		}

		/// <summary>
		/// Format an include of the given file
		/// </summary>
		/// <param name="FromDirectory">The directory containing the file with the #include directive</param>
		/// <param name="IncludeFile">File to include</param>
		/// <param name="IncludePaths">Directories to base relative include paths from</param>
		/// <param name="SystemIncludePaths">Directories to base system include paths from</param>
		/// <returns>Formatted include path, with surrounding quotes</returns>
		public static bool TryFormatInclude(DirectoryReference FromDirectory, FileReference IncludeFile, IEnumerable<DirectoryReference> IncludePaths, IEnumerable<DirectoryReference> SystemIncludePaths, out string IncludeText)
		{
			// Generated headers are always included without a path
			if(IncludeFile.FullName.Contains(".generated.") || IncludeFile.FullName.EndsWith("Classes.h"))
			{
				IncludeText = String.Format("\"{0}\"", IncludeFile.GetFileName());
				return true;
			}

			// Try to write an include relative to one of the system include paths
			foreach(DirectoryReference SystemIncludePath in SystemIncludePaths)
			{
				if(IncludeFile.IsUnderDirectory(SystemIncludePath))
				{
					IncludeText = String.Format("<{0}>", IncludeFile.MakeRelativeTo(SystemIncludePath).Replace('\\', '/'));
					return true;
				}
			}

			// Try to write an include relative to one of the standard include paths
			foreach(DirectoryReference IncludePath in IncludePaths)
			{
				if(IncludeFile.IsUnderDirectory(IncludePath))
				{
					IncludeText = String.Format("\"{0}\"", IncludeFile.MakeRelativeTo(IncludePath).Replace('\\', '/'));
					return true;
				}
			}

			// Try to write an include relative to the file
			if (IncludeFile.IsUnderDirectory(FromDirectory))
			{
				IncludeText = String.Format("\"{0}\"", IncludeFile.MakeRelativeTo(FromDirectory).Replace('\\', '/'));
				return true;
			}

			// HACK: VsPerf.h is in the compiler environment, but the include path is added directly
			if (IncludeFile.FullName.EndsWith("\\PerfSDK\\VSPerf.h", StringComparison.OrdinalIgnoreCase))
			{
				IncludeText = "\"VSPerf.h\"";
				return true;
			}

			// HACK: public Paper2D header in classes folder including private Paper2D header
			if(IncludeFile.FullName.IndexOf("\\Paper2D\\Private\\", StringComparison.OrdinalIgnoreCase) != -1)
			{
				IncludeText = "\"" + IncludeFile.GetFileName() + "\"";
				return true;
			}
			if(IncludeFile.FullName.IndexOf("\\OnlineSubsystemUtils\\Private\\", StringComparison.OrdinalIgnoreCase) != -1)
			{
				IncludeText = "\"" + IncludeFile.GetFileName() + "\"";
				return true;
			}

			// HACK: including private headers from public headers in the same module
			int PrivateIdx = IncludeFile.FullName.IndexOf("\\Private\\", StringComparison.OrdinalIgnoreCase);
			if(PrivateIdx != -1)
			{
				DirectoryReference BaseDir = new DirectoryReference(IncludeFile.FullName.Substring(0, PrivateIdx));
				if(FromDirectory.IsUnderDirectory(BaseDir))
				{
					IncludeText = "\"" + IncludeFile.MakeRelativeTo(FromDirectory).Replace('\\', '/') + "\"";
					return true;
				}
			}

			// Otherwise we don't know where it came from
			IncludeText = null;
			return false;
		}

		/// <summary>
		/// Converts a source file to a string for debugging
		/// </summary>
		/// <returns>The full path to the file</returns>
		public override string ToString()
		{
			return Location.ToString();
		}
	}
}
