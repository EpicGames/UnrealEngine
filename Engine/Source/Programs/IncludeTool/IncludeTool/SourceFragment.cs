// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Represents an fragment of a source file consisting of a sequence of includes followed by some arbitrary directives or source code.
	/// </summary>
	[Serializable]
	class SourceFragment
	{
		/// <summary>
		/// The file that this fragment is part of
		/// </summary>
		public SourceFile File;

		/// <summary>
		/// Index into the file's markup array of the start of this fragment (inclusive)
		/// </summary>
		public int MarkupMin;

		/// <summary>
		/// Index into the file's markup array of the end of this fragment (exclusive). Set to zero for external files.
		/// </summary>
		public int MarkupMax;

		/// <summary>
		/// Unique name assigned to this fragment.
		/// </summary>
		public string UniqueName;

		/// <summary>
		/// Location of this fragment on disk. May be the original file in the case the single fragment for an external file, or a written out extract corresponding to the markup range.
		/// </summary>
		public FileReference Location;

		/// <summary>
		/// Set of fragments which this fragment is dependent on
		/// </summary>
		public HashSet<SourceFragment> Dependencies;

		/// <summary>
		/// All the imported symbols for this fragment
		/// </summary>
		public Dictionary<Symbol, SymbolReferenceType> ReferencedSymbols;

		/// <summary>
		/// SHA1 digest of this fragment's text on disk
		/// </summary>
		public string Digest;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InFile">The source file that this fragment is part of</param>
		/// <param name="InMarkupMin">Index into the file's markup array of the start of this fragment (inclusive)</param>
		/// <param name="InMarkupMax">Index into the file's markup array of the end of this fragment (exclusive)</param>
		public SourceFragment(SourceFile File, int MarkupMin, int MarkupMax)
		{
			this.File = File;
			this.MarkupMin = MarkupMin;
			this.MarkupMax = MarkupMax;
		}

		/// <summary>
		/// Find a flat list of fragments included by this file
		/// </summary>
		/// <returns>List of included fragments, in order</returns>
		public List<SourceFragment> FindIncludedFragments()
		{
			List<SourceFragment> IncludedFragments = new List<SourceFragment>();
			FindIncludedFragments(IncludedFragments, null, new HashSet<SourceFile>());
			return IncludedFragments;
		}

		/// <summary>
		/// Find all the fragments included by this fragment, including the current one
		/// </summary>
		/// <param name="IncludedFragments">List to receive the included fragments</param>
		/// <param name="IncludeStackChanges">Optional list to store changes to the include stack, and the count of fragments at that point. Records with a file set to null indicate that the end of the topmost file.</param>
		/// <param name="UniqueIncludedFiles">Set of files with header guards that have already been included</param>
		public void FindIncludedFragments(List<SourceFragment> IncludedFragments, List<Tuple<int, SourceFile>> IncludeStackChanges, HashSet<SourceFile> UniqueIncludedFiles)
		{
			if((File.Flags & SourceFileFlags.Inline) == 0)
			{
				// Add all the included files into the tree, and save the depth at each step
				for(int MarkupIdx = MarkupMin; MarkupIdx < MarkupMax; MarkupIdx++)
				{
					PreprocessorMarkup Item = File.Markup[MarkupIdx];
					if(Item.Type == PreprocessorMarkupType.Include && Item.IsActive && Item.IncludedFile != null)
					{
						SourceFile IncludedFile = Item.IncludedFile;
						if(!IncludedFile.HasHeaderGuard || UniqueIncludedFiles.Add(IncludedFile))
						{
							IncludedFile.FindIncludedFragments(IncludedFragments, IncludeStackChanges, UniqueIncludedFiles);
						}
					}
				}

				// Add the current fragment
				IncludedFragments.Add(this);
			}
		}

		/// <summary>
		/// The text location of the start of this fragment
		/// </summary>
		public TextLocation MinLocation
		{
			get { return (MarkupMin < File.Markup.Length)? File.Markup[MarkupMin].Location : File.Text.End; }
		}

		/// <summary>
		/// The text location of the end of this fragment
		/// </summary>
		public TextLocation MaxLocation
		{
			get { return (MarkupMax < File.Markup.Length)? File.Markup[MarkupMax].Location : File.Text.End; }
		}

		/// <summary>
		/// Writes this fragment to a file
		/// </summary>
		/// <param name="Writer">Writer to receive output text</param>
		public void Write(TextWriter Writer)
		{
			TextBuffer Text = File.Text;

			// Write all the relevant bits from this file
			TextLocation Location = TextLocation.Origin;
			for(int Idx = File.BodyMinIdx; Idx < File.BodyMaxIdx; Idx++)
			{
				PreprocessorMarkup Markup = File.Markup[Idx];
				if((Idx >= MarkupMin && Idx < MarkupMax) || Markup.IsConditionalPreprocessorDirective())
				{
					if(Markup.Type != PreprocessorMarkupType.Include)
					{
						TextLocation EndLocation = (Idx + 1 < File.Markup.Length)? File.Markup[Idx + 1].Location : File.Text.End;
						WriteLinesCommented(Writer, Text, Location, Markup.Location);
						WriteLines(Writer, Text, Markup.Location, EndLocation);
						Location = EndLocation;
					}
					else if(!Markup.IsActive)
					{
						WriteLinesCommented(Writer, Text, Location, Markup.Location);
						Writer.WriteLine("#error Unexpected include: {0}", Token.Format(Markup.Tokens));
						Location = new TextLocation(Markup.Location.LineIdx + 1, 0);
					}
					else if(Markup.IncludedFile != null && (Markup.IncludedFile.Flags & SourceFileFlags.Inline) != 0)
					{
						TextLocation EndLocation = (Idx + 1 < File.Markup.Length)? File.Markup[Idx + 1].Location : File.Text.End;
						WriteLinesCommented(Writer, Text, Location, Markup.Location);
						Writer.Write("#include \"{0}\"", Markup.IncludedFile.Location.FullName);
						WriteLinesCommented(Writer, Text, Markup.Location, EndLocation);
						Location = EndLocation;
					}
				}
			}

			// Comment out the rest of the file
			WriteLinesCommented(Writer, Text, Location, Text.End);
		}

		/// <summary>
		/// Utility function to write out a portion of a TextBuffer to a TextWriter
		/// </summary>
		/// <param name="Writer">The writer to receive the extracted text</param>
		/// <param name="Text">The text buffer being read from</param>
		/// <param name="Location">Start location of the text to extract</param>
		/// <param name="EndLocation">End location of the text to extract</param>
		static void WriteLines(TextWriter Writer, TextBuffer Text, TextLocation Location, TextLocation EndLocation)
		{
			string[] TextLines = Text.Extract(Location, EndLocation);
			for(int Idx = 0; Idx < TextLines.Length - 1; Idx++)
			{
				Writer.WriteLine(TextLines[Idx]);
			}
			Writer.Write(TextLines[TextLines.Length - 1]);
		}

		/// <summary>
		/// Utility function to write out a portion of a TextBuffer to a TextWriter as comments
		/// </summary>
		/// <param name="Writer">The writer to receive the extracted text</param>
		/// <param name="Text">The text buffer being read from</param>
		/// <param name="Location">Start location of the text to extract</param>
		/// <param name="EndLocation">End location of the text to extract</param>
		static void WriteLinesCommented(TextWriter Writer, TextBuffer Text, TextLocation Location, TextLocation EndLocation)
		{
			string[] CommentLines = Text.Extract(Location, EndLocation);
			for(int Idx = 0; Idx < CommentLines.Length - 1; Idx++)
			{
				string CommentLine = CommentLines[Idx];
				if(CommentLine.EndsWith("\\")) CommentLine = CommentLine.Substring(CommentLine.Length - 1);
				Writer.WriteLine("// XXX {0}", CommentLine);
			}
		}

		/// <summary>
		/// Summarize this fragment for the debugger
		/// </summary>
		/// <returns>String representation of this fragment for the debugger</returns>
		public override string ToString()
		{
			if(Location != null)
			{
				return Location.ToString();
			}
			else if(MarkupMax == 0)
			{
				return File.ToString();
			}
			else
			{
				return String.Format("{0}: {1}-{2}", File.ToString(), File.Markup[MarkupMin].Location.LineIdx + 1, File.Markup[MarkupMax - 1].EndLocation.LineIdx + 1);
			}
		}
	}
}
