// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.Serialization;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Fast, forward-only binary serializer which can handle arbitrary object hierarchies (including circular references).
	/// </summary>
	static class BinarySerializer
	{
		/// <summary>
		/// Cached lookup of type to field array for serialized types
		/// </summary>
		static Dictionary<Type, FieldInfo[]> AllFieldsMap = new Dictionary<Type, FieldInfo[]>();

		/// <summary>
		/// Serialize an object hierarchy to a file, starting at the given root object
		/// </summary>
		/// <param name="RootObject">The root object to serialize</param>
		/// <param name="FileName">Path to the output file</param>
		public static void Serialize(object RootObject, string FileName)
		{
			using (FileStream OutputStream = File.Open(FileName, FileMode.Create, FileAccess.Write, FileShare.Read))
			{
				Serialize(RootObject, OutputStream);
			}
		}

		/// <summary>
		/// Serialize an object hierarchy to a stream, starting at the given root object
		/// </summary>
		/// <param name="RootObject">The root object to serialize</param>
		/// <param name="FileName">Output stream for the binary data</param>
		public static void Serialize(object RootObject, Stream OutputStream)
		{
			HashList<Type> Types = new HashList<Type>();
			HashList<object> Instances = new HashList<object>();
			using (BinaryWriter Writer = new BinaryWriter(OutputStream, Encoding.UTF8, true))
			{
				SerializeInstance(RootObject, Types, Instances, Writer);
			}
		}

		/// <summary>
		/// Deserialize an object hierarchy from the given file
		/// </summary>
		/// <param name="FileName">Path to the file to read from</param>
		/// <returns>The deserialized object</returns>
		public static object Deserialize(string FileName)
		{
			using (FileStream InputStream = File.Open(FileName, FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				return Deserialize(InputStream);
			}
		}

		/// <summary>
		/// Deserialize an object hierarchy from the given stream
		/// </summary>
		/// <param name="FileName">Path to the file to read from</param>
		/// <returns>The deserialized object</returns>
		public static object Deserialize(Stream InputStream)
		{
			List<Type> Types = new List<Type>();
			List<object> Instances = new List<object>();
			using (BinaryReader Reader = new BinaryReader(InputStream, Encoding.UTF8, true))
			{
				return DeserializeInstance(Types, Instances, Reader);
			}
		}


		/// <summary>
		/// Serialize an object instance to a BinaryWriter
		/// </summary>
		/// <param name="ThisObject">The object to serialize</param>
		/// <param name="Types">List of types which have been serialized before. If the type of this object is new, it will be added to the list.</param>
		/// <param name="Instances">List of object instances which have already been serialized. The given object must NOT be in this list.</param>
		/// <param name="Writer">The writer for output data</param>
		static void SerializeInstance(object ThisObject, HashList<Type> Types, HashList<object> Instances, BinaryWriter Writer)
		{
			Type ThisType = ThisObject.GetType();

			// Write the type
			int TypeIdx = Types.IndexOf(ThisType);
			if (TypeIdx == -1)
			{
				Types.Add(ThisType);
				Writer.Write((int)(Types.Count - 1));
				Writer.Write(ThisType.FullName);
			}
			else
			{
				Writer.Write(TypeIdx);
			}

			// Now write the instance data
			Instances.Add(ThisObject);
			if (ThisType.IsArray)
			{
				Array ThisArrayObject = (Array)ThisObject;
				for (int DimensionIdx = 0; DimensionIdx < ThisArrayObject.Rank; DimensionIdx++)
				{
					Writer.Write(ThisArrayObject.GetLength(DimensionIdx));
				}

				Type ElementType = ThisArrayObject.GetType().GetElementType();
				foreach (int[] Indices in EnumerateArrayIndices(ThisArrayObject))
				{
					SerializeElement(ElementType, ThisArrayObject.GetValue(Indices), Types, Instances, Writer);
				}
			}
			else if (ThisType == typeof(string))
			{
				Writer.Write((string)ThisObject);
			}
			else
			{
				SerializeObjectFields(ThisObject, FindAllFields(ThisType), Types, Instances, Writer);
			}
		}

		/// <summary>
		/// Deserialize an object instance from a binary stream.
		/// </summary>
		/// <param name="Types">List of all types which have been serialized so far.</param>
		/// <param name="Instances">List of object instances which have been serialized so far.</param>
		/// <param name="Reader">Binary reader for input data</param>
		/// <returns>Deserialized object instance</returns>
		static object DeserializeInstance(List<Type> Types, List<object> Instances, BinaryReader Reader)
		{
			Type ThisType;

			// Get the type
			int TypeIdx = Reader.ReadInt32();
			if (TypeIdx == Types.Count)
			{
				string TypeName = Reader.ReadString();
				ThisType = Type.GetType(TypeName);
				Types.Add(ThisType);
			}
			else
			{
				ThisType = Types[TypeIdx];
			}

			// Create and read the object
			if (ThisType.IsArray)
			{
				int[] Lengths = new int[ThisType.GetArrayRank()];
				for (int Idx = 0; Idx < Lengths.Length; Idx++)
				{
					Lengths[Idx] = Reader.ReadInt32();
				}

				Array ThisArrayObject = Array.CreateInstance(ThisType.GetElementType(), Lengths);
				Instances.Add(ThisArrayObject);

				Type ElementType = ThisArrayObject.GetType().GetElementType();
				foreach (int[] Indices in EnumerateArrayIndices(ThisArrayObject))
				{
					ThisArrayObject.SetValue(DeserializeElement(ElementType, Types, Instances, Reader), Indices);
				}

				return ThisArrayObject;
			}
			else if (ThisType == typeof(string))
			{
				string ThisStringObject = Reader.ReadString();
				Instances.Add(ThisStringObject);
				return ThisStringObject;
			}
			else
			{
				object ThisObject = FormatterServices.GetUninitializedObject(ThisType);
				Instances.Add(ThisObject);
				DeserializeObjectFields(ThisObject, FindAllFields(ThisType), Types, Instances, Reader);
				return ThisObject;
			}
		}

		/// <summary>
		/// Serialize fields within an object instance
		/// </summary>
		/// <param name="Instance">The object to serializer</param>
		/// <param name="Fields">Array of fields within this object</param>
		/// <param name="Types">Lookup of known types</param>
		/// <param name="Instances">Lookup of known object instances</param>
		/// <param name="Writer">Binary writer for output data</param>
		static void SerializeObjectFields(object Instance, FieldInfo[] Fields, HashList<Type> Types, HashList<object> Instances, BinaryWriter Writer)
		{
			foreach (FieldInfo Field in Fields)
			{
				object Element = Field.GetValue(Instance);
				SerializeElement(Field.FieldType, Element, Types, Instances, Writer);
			}
		}

		/// <summary>
		/// Deserialize fields into an object instance
		/// </summary>
		/// <param name="Instance">The object to serializer</param>
		/// <param name="Fields">Array of fields within this object</param>
		/// <param name="Types">Lookup of known types</param>
		/// <param name="Instances">Lookup of known object instances</param>
		/// <param name="Reader">Binary reader for input data</param>
		static void DeserializeObjectFields(object Instance, FieldInfo[] Fields, List<Type> Types, List<object> Instances, BinaryReader Reader)
		{
			foreach (FieldInfo Field in Fields)
			{
				object Element = DeserializeElement(Field.FieldType, Types, Instances, Reader);
				Field.SetValue(Instance, Element);
			}
		}

		/// <summary>
		/// Serialize a single value or array element
		/// </summary>
		/// <param name="ElementType">Type of the element</param>
		/// <param name="ElementValue">Value to serialize</param>
		/// <param name="Types">List of known types</param>
		/// <param name="Instances">List of known object instances</param>
		/// <param name="Writer">Writer for the output data</param>
		static void SerializeElement(Type ElementType, object ElementValue, HashList<Type> Types, HashList<object> Instances, BinaryWriter Writer)
		{
			if (ElementType == typeof(Boolean))
			{
				Writer.Write((Boolean)ElementValue);
			}
			else if (ElementType == typeof(Int32))
			{
				Writer.Write((Int32)ElementValue);
			}
			else if (ElementType.IsEnum)
			{
				Type UnderlyingType = ElementType.GetEnumUnderlyingType();
				SerializeElement(UnderlyingType, Convert.ChangeType(ElementValue, UnderlyingType), Types, Instances, Writer);
			}
			else if (ElementType.IsClass || ElementType.IsArray || ElementType.IsInterface)
			{
				if (ElementValue == null)
				{
					Writer.Write(-1);
				}
				else
				{
					int Index = Instances.IndexOf(ElementValue);
					if (Index == -1)
					{
						Writer.Write(Instances.Count);
						SerializeInstance(ElementValue, Types, Instances, Writer);
					}
					else
					{
						Writer.Write(Index);
					}
				}
			}
			else
			{
				FieldInfo[] Fields = FindAllFields(ElementType);
				if (Fields.Length > 0)
				{
					SerializeObjectFields(ElementValue, Fields, Types, Instances, Writer);
				}
				else
				{
					throw new NotImplementedException();
				}
			}
		}

		/// <summary>
		/// Deserialize a single value or array element
		/// </summary>
		/// <param name="ElementType">Type of the element</param>
		/// <param name="ElementValue">Value to serialize</param>
		/// <param name="Types">List of known types</param>
		/// <param name="Instances">List of known object instances</param>
		/// <param name="Reader">Binary reader for input data</param>
		/// <returns>The new object</returns>
		static object DeserializeElement(Type ElementType, List<Type> Types, List<object> Instances, BinaryReader Reader)
		{
			if (ElementType == typeof(Boolean))
			{
				return Reader.ReadBoolean();
			}
			else if (ElementType == typeof(Int32))
			{
				return Reader.ReadInt32();
			}
			else if (ElementType.IsEnum)
			{
				Type UnderlyingType = ElementType.GetEnumUnderlyingType();
				return Convert.ChangeType(DeserializeElement(UnderlyingType, Types, Instances, Reader), UnderlyingType);
			}
			else if (ElementType.IsClass || ElementType.IsArray || ElementType.IsInterface)
			{
				int Index = Reader.ReadInt32();
				if (Index == -1)
				{
					return null;
				}
				else if (Index == Instances.Count)
				{
					return DeserializeInstance(Types, Instances, Reader);
				}
				else
				{
					return Instances[Index];
				}
			}
			else
			{
				FieldInfo[] Fields = FindAllFields(ElementType);
				if (Fields.Length > 0)
				{
					object BoxedStruct = FormatterServices.GetUninitializedObject(ElementType);
					DeserializeObjectFields(BoxedStruct, Fields, Types, Instances, Reader);
					return BoxedStruct;
				}
				else
				{
					throw new NotImplementedException();
				}
			}
		}

		/// <summary>
		/// Enumerate all the indices of an array, regardless of its rank.
		/// </summary>
		/// <param name="ThisArrayObject">The array to enumerate indices for</param>
		/// <returns>Sequence of indices, expressed as an array of integers corresponding to each dimension of the array</returns>
		static IEnumerable<int[]> EnumerateArrayIndices(Array ThisArrayObject)
		{
			int[] Indices = new int[ThisArrayObject.Rank];
			while (Indices[0] < ThisArrayObject.GetLength(0))
			{
				yield return Indices;

				for (int DimensionIdx = Indices.Length - 1; DimensionIdx >= 0; DimensionIdx--)
				{
					Indices[DimensionIdx]++;
					if (DimensionIdx == 0 || Indices[DimensionIdx] < ThisArrayObject.GetLength(DimensionIdx))
					{
						break;
					}
					Indices[DimensionIdx] = 0;
				}
			}
		}

		/// <summary>
		/// Gets an array of serialized fields for the given type
		/// </summary>
		/// <param name="InType">The type to find serializable fields for</param>
		/// <returns>Array of fields</returns>
		static FieldInfo[] FindAllFields(Type InType)
		{
			FieldInfo[] AllFields;
			if (!AllFieldsMap.TryGetValue(InType, out AllFields))
			{
				AllFields = FindAllFieldsUncached(InType);
				AllFieldsMap.Add(InType, AllFields);
			}
			return AllFields;
		}

		/// <summary>
		/// Gets an array of serialized fields for the given type, without using the field cache
		/// </summary>
		/// <param name="InType">The type to find serializable fields for</param>
		/// <returns>Array of fields</returns>
		static FieldInfo[] FindAllFieldsUncached(Type InType)
		{
			Dictionary<RuntimeFieldHandle, FieldInfo> FieldMap = new Dictionary<RuntimeFieldHandle, FieldInfo>();
			for (Type CurrentType = InType; CurrentType != null; CurrentType = CurrentType.BaseType)
			{
				FieldInfo[] Fields = CurrentType.GetFields(BindingFlags.Instance | BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.DeclaredOnly);
				foreach (FieldInfo Field in Fields)
				{
					if (!Field.Attributes.HasFlag(FieldAttributes.NotSerialized))
					{
						FieldMap[Field.FieldHandle] = Field;
					}
				}
			}
			return FieldMap.Values.ToArray();
		}
	}
}
