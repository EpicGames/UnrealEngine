// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Implementation of a TextWriter which buffers characters written to it, and can be copied to another TextWriter at a later time.
	/// </summary>
	class BufferedTextWriter : LineBasedTextWriter
	{
		/// <summary>
		/// List of buffered lines
		/// </summary>
		List<string> Lines = new List<string>();

		/// <summary>
		/// Default constructor
		/// </summary>
		public BufferedTextWriter()
		{
		}

		/// <summary>
		/// Construct a BufferedTextWriter with the given initial array of lines
		/// </summary>
		/// <param name="InLines">Initial array of lines</param>
		public BufferedTextWriter(string[] InLines)
		{
			Lines.AddRange(InLines);
		}

		/// <summary>
		/// Clear out the contents of the buffer
		/// </summary>
		public void Clear()
		{
			Lines.Clear();
		}

		/// <summary>
		/// Copies the current contents of the BufferedTextWriter to another TextWriter
		/// </summary>
		/// <param name="Other">The TextWriter to copy the buffered text to</param>
		public void CopyTo(LineBasedTextWriter Other, string Prefix)
		{
			for (int Idx = 0; Idx < Lines.Count; Idx++)
			{
				Other.WriteLine("{0}{1}", Prefix, Lines[Idx]);
			}
		}

		/// <summary>
		/// Write a single character to the buffer
		/// </summary>
		/// <param name="Character">Character to write</param>
		public override void WriteLine(string Line)
		{
			Lines.Add(Line);
		}

		/// <summary>
		/// Returns the current buffered contents of the TextWriter as a string
		/// </summary>
		/// <returns>String containing the current buffered text</returns>
		public override string ToString()
		{
			return String.Join("\n", Lines);
		}
	}
}
