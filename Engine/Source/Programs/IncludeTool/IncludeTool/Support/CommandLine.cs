// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Attribute to markup fields which can be set by command line argument
	/// </summary>
	public class CommandLineOptionAttribute : Attribute
	{
		/// <summary>
		/// The argument name. Uses the field name if not set.
		/// </summary>
		public string Name;

		/// <summary>
		/// If set, marks that the argument is required.
		/// </summary>
		public bool IsRequired;

		/// <summary>
		/// For multi-value fields, indicates a separator character between arguments
		/// </summary>
		public char Separator;
	}

	/// <summary>
	/// Utility functions for parsing command line options
	/// </summary>
	static class CommandLine
	{
		/// <summary>
		/// Parse the options from the given command line into the given options, using CommandLineOption attributes.
		/// </summary>
		/// <param name="Args">The arguments to parse</param>
		/// <param name="Options">Object to receive the parsed options</param>
		/// <returns>True if all options were parsed, false otherwise</returns>
		public static bool Parse(string[] Args, object Options)
		{
			bool bResult = true;

			// Parse out all the arguments we can
			List<string> RemainingArgs = new List<string>(Args);
			foreach(FieldInfo Field in Options.GetType().GetFields())
			{
				CommandLineOptionAttribute OptionAttribute = Field.GetCustomAttribute<CommandLineOptionAttribute>();
				if(OptionAttribute != null)
				{
					string Name = OptionAttribute.Name ?? Field.Name;
					if (Field.FieldType == typeof(bool))
					{
						string Flag = String.Format("-{0}", Name);
						if(RemainingArgs.RemoveAll(x => x.Equals(Flag, StringComparison.OrdinalIgnoreCase)) > 0)
						{
							Field.SetValue(Options, true);
						}
					}
					else
					{
						string Prefix = String.Format("-{0}=", Name);

						// Find all the values with this prefix
						List<string> Values = new List<string>();
						for (int Idx = 0; Idx < RemainingArgs.Count; Idx++)
						{
							if (RemainingArgs[Idx].StartsWith(Prefix, StringComparison.OrdinalIgnoreCase))
							{
								string Suffix = RemainingArgs[Idx].Substring(Prefix.Length);
								if (OptionAttribute.Separator != 0)
								{
									Values.AddRange(Suffix.Split(OptionAttribute.Separator));
								}
								else
								{
									Values.Add(Suffix);
								}
								RemainingArgs.RemoveAt(Idx--);
							}
						}

						// Check if the field type implements ICollection<>. If so, we can take multiple values.
						Type CollectionType = null;
						foreach (Type InterfaceType in Field.FieldType.GetInterfaces())
						{
							if (InterfaceType.IsGenericType && InterfaceType.GetGenericTypeDefinition() == typeof(ICollection<>))
							{
								CollectionType = InterfaceType;
								break;
							}
						}

						// Assign the values to the field
						if(Values.Count == 0)
						{
							if(OptionAttribute.IsRequired)
							{
								Console.WriteLine("Missing parameter {0}...", Prefix);
								bResult = false;
							}
						}
						else
						{
							if (CollectionType == null)
							{
								if (Values.Count > 1)
								{
									Console.WriteLine("Multiple values specified for {0}...", Prefix);
									bResult = false;
								}
								else if(Field.FieldType.IsEnum)
								{
									try
									{
										object TypedValue = Enum.Parse(Field.FieldType, Values[0], true);
										Field.SetValue(Options, TypedValue);
									}
									catch(ArgumentException)
									{
										Console.WriteLine("Invalid argument for '{0}'. Expected {1}.", Name, String.Join("/", Enum.GetNames(Field.FieldType)));
										bResult = false;
									}
								}
								else
								{
									object TypedValue = Convert.ChangeType(Values[0], Field.FieldType);
									Field.SetValue(Options, TypedValue);
								}
							}
							else
							{
								foreach (string Value in Values)
								{
									object TypedValue = Convert.ChangeType(Value, CollectionType.GenericTypeArguments[0]);
									CollectionType.InvokeMember("Add", BindingFlags.InvokeMethod, null, Field.GetValue(Options), new object[] { TypedValue });
								}
							}
						}
					}
				}
			}

			// Check there's nothing left over that we couldn't parse
			if (RemainingArgs.Count > 0)
			{
				foreach(string RemainingArg in RemainingArgs)
				{
					Console.WriteLine("Invalid argument: {0}", RemainingArg);
				}
				bResult = false;
			}

			return bResult;
		}
	}
}
