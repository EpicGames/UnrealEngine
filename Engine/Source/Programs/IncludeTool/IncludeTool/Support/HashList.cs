// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Implements an ordered list of unique elements which can quickly be queried for the existence and index of an existing item
	/// </summary>
	/// <typeparam name="T">Type to be stored in the container</typeparam>
	[Serializable]
	[DebuggerDisplay("Count = {Count}")]
	[DebuggerTypeProxy("System.Collections.Generic.Mscorlib_CollectionDebugView`1, mscorlib, Version=2.0.0.0, Culture=neutral, PublicKeyToken=b77a5c561934e089")]
	class HashList<T> : IList<T>, ICollection<T>, IReadOnlyList<T>, IReadOnlyCollection<T>, IEnumerable<T>
	{
		/// <summary>
		/// Ordered list of items
		/// </summary>
		List<T> Items = new List<T>();

		/// <summary>
		/// Map from element to its index
		/// </summary>
		Dictionary<T, int> ItemToIndex = new Dictionary<T, int>();

		/// <summary>
		/// Construct an empty HashList
		/// </summary>
		public HashList()
		{
		}

		/// <summary>
		/// Construct a HashList from the given input collection
		/// </summary>
		/// <param name="Collection">Collection of elements</param>
		public HashList(IEnumerable<T> Collection)
		{
			foreach (T Item in Collection)
			{
				Add(Item);
			}
		}

		/// <summary>
		/// The number of elements in the collection
		/// </summary>
		public int Count
		{
			get { return Items.Count; }
		}

		/// <summary>
		/// Get or set an element at the given index. Throws an exception if the item already exists.
		/// </summary>
		/// <param name="Index">Index of the item to get or set</param>
		/// <returns>The item at the given index</returns>
		public T this[int Index]
		{
			get
			{
				return Items[Index];
			}
			set
			{
				ItemToIndex.Add(value, Index);
				ItemToIndex.Remove(Items[Index]);
				Items[Index] = value;
			}
		}

		/// <summary>
		/// Returns false.
		/// </summary>
		public bool IsReadOnly
		{
			get { return false; }
		}

		/// <summary>
		/// Clear the list of items.
		/// </summary>
		public void Clear()
		{
			Items.Clear();
			ItemToIndex.Clear();
		}

		/// <summary>
		/// Adds an item to the end of the list
		/// </summary>
		/// <param name="Item">Item to add</param>
		void ICollection<T>.Add(T Item)
		{
			Insert(Items.Count, Item);
		}

		/// <summary>
		/// Add an item to the end of the list
		/// </summary>
		/// <param name="Item">Item to add</param>
		public bool Add(T Item)
		{
			if(Contains(Item))
			{
				return false;
			}

			Insert(Items.Count, Item);
			return true;
		}

		/// <summary>
		/// Insert an item at the given index
		/// </summary>
		/// <param name="Index">Index to insert at</param>
		/// <param name="Item">Item to insert</param>
		public void Insert(int Index, T Item)
		{
			// Add the item to the dictionary first; it may throw
			ItemToIndex.Add(Item, Index);
			Items.Insert(Index, Item);
			RenumberAfter(Index + 1);
		}

		/// <summary>
		/// Removes the given item from the container
		/// </summary>
		/// <param name="Item">Item to remove</param>
		/// <returns>True if the item was removed, false if it was not present</returns>
		public bool Remove(T Item)
		{
			int Index;
			if (ItemToIndex.TryGetValue(Item, out Index))
			{
				RemoveAt(Index);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Removes an item at the given index
		/// </summary>
		/// <param name="Index">Index of the item to remove</param>
		public void RemoveAt(int Index)
		{
			Items.RemoveAt(Index);
			RenumberAfter(Index);
		}

		/// <summary>
		/// Remove all items matching a predicate
		/// </summary>
		/// <param name="Predicate">The predicate to test each item against</param>
		public void RemoveWhere(Predicate<T> Predicate)
		{
			for(int Idx = 0; Idx < Items.Count; Idx++)
			{
				if(Predicate(Items[Idx]))
				{
					Remove(Items[Idx]);
					Idx--;
				}
			}
		}

		/// <summary>
		/// Renumber all the elements at the given index or above. Required after reordering the item list.
		/// </summary>
		/// <param name="FirstIndex">The first item index to renumber</param>
		void RenumberAfter(int FirstIndex)
		{
			for (int NewIndex = FirstIndex; NewIndex < Items.Count; NewIndex++)
			{
				T Item = Items[NewIndex];
				ItemToIndex[Item] = NewIndex;
			}
		}

		/// <summary>
		/// Checks whether a given item belongs to the container
		/// </summary>
		/// <param name="Item">Item to look for</param>
		/// <returns>True if the item is in the container, false otherwise</returns>
		public bool Contains(T Item)
		{
			return ItemToIndex.ContainsKey(Item);
		}

		/// <summary>
		/// Gets the index of the given item.
		/// </summary>
		/// <param name="Item">Item to look for.</param>
		/// <returns>Index of the item, or -1 if it's not present.</returns>
		public int IndexOf(T Item)
		{
			int Index;
			if (ItemToIndex.TryGetValue(Item, out Index))
			{
				return Index;
			}
			return -1;
		}

		/// <summary>
		/// Gets an ordered enumerator of the containers contents.
		/// </summary>
		/// <returns>Enumerator for the container</returns>
		IEnumerator IEnumerable.GetEnumerator()
		{
			return Items.GetEnumerator();
		}

		/// <summary>
		/// Gets an ordered enumerator of the containers contents.
		/// </summary>
		/// <returns>Enumerator for the container</returns>
		public IEnumerator<T> GetEnumerator()
		{
			return Items.GetEnumerator();
		}

		/// <summary>
		/// Add all the given items into the list
		/// </summary>
		/// <param name="Items">Items to add</param>
		public void UnionWith(IEnumerable<T> Items)
		{
			foreach(T Item in Items)
			{
				Add(Item);
			}
		}

		/// <summary>
		/// Converts the list to an array.
		/// </summary>
		/// <returns>Array of elements</returns>
		public T[] ToArray()
		{
			return Items.ToArray();
		}

		/// <summary>
		/// Copy the container to an array
		/// </summary>
		/// <param name="TargetArray">Target array to receive the contents of this container</param>
		/// <param name="TargetIndex">Index of the target array to receive the first element</param>
		public void CopyTo(T[] TargetArray, int TargetIndex)
		{
			Items.CopyTo(TargetArray, TargetIndex);
		}
	}
}
