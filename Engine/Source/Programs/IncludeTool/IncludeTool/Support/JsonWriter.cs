// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	public class JsonWriter : IDisposable
	{
		StreamWriter Writer;
		bool bRequiresComma;
		string Indent;

		public JsonWriter(string FileName)
		{
			Writer = new StreamWriter(FileName);
			Indent = "";
		}

		public void Dispose()
		{
			Writer.Dispose();
		}

		public void WriteObjectStart()
		{
			WriteCommaNewline();

			Writer.Write(Indent);
			Writer.Write("{");

			Indent += "\t";
			bRequiresComma = false;
		}

		public void WriteObjectStart(string ObjectName)
		{
			WriteCommaNewline();

			Writer.Write("{0}\"{1}\" : ", Indent, ObjectName);

			bRequiresComma = false;

			WriteObjectStart();
		}

		public void WriteObjectEnd()
		{
			Indent = Indent.Substring(0, Indent.Length - 1);

			Writer.WriteLine();
			Writer.Write(Indent);
			Writer.Write("}");

			bRequiresComma = true;
		}

		public void WriteArray(string ArrayName, IEnumerable<string> Values)
		{
			WriteArrayStart(ArrayName);
			foreach(string Value in Values)
			{
				WriteValue(Value);
			}
			WriteArrayEnd();
		}

		public void WriteArray(string ArrayName, IEnumerable<int> Values)
		{
			WriteArrayStart(ArrayName);
			foreach(int Value in Values)
			{
				WriteValue(Value);
			}
			WriteArrayEnd();
		}

		public void WriteArrayStart(string ArrayName)
		{
			WriteCommaNewline();

			Writer.WriteLine("{0}\"{1}\" :", Indent, ArrayName);
			Writer.Write("{0}[", Indent);

			Indent += "\t";
			bRequiresComma = false;
		}

		public void WriteArrayEnd()
		{
			Indent = Indent.Substring(0, Indent.Length - 1);

			Writer.WriteLine();
			Writer.Write("{0}]", Indent);

			bRequiresComma = true;
		}

		public void WriteValue(string Value)
		{
			WriteCommaNewline();

			Writer.Write(Indent);
			WriteEscapedString(Value);

			bRequiresComma = true;
		}

		public void WriteValue(int Value)
		{
			WriteCommaNewline();
			Writer.Write("{0}{1}", Indent, Value);
			bRequiresComma = true;
		}

		public void WriteValue(double Value)
		{
			WriteCommaNewline();
			Writer.Write("{0}{1}", Indent, Value);
			bRequiresComma = true;
		}

		public void WriteValue(bool Value)
		{
			WriteCommaNewline();
			Writer.Write("{0}{1}", Indent, Value? "true" : "false");
			bRequiresComma = true;
		}

		public void WriteValue(string Name, string Value)
		{
			WriteCommaNewline();

			Writer.Write("{0}\"{1}\" : ", Indent, Name);
			WriteEscapedString(Value);

			bRequiresComma = true;
		}

		public void WriteValue(string Name, int Value)
		{
			WriteValueInternal(Name, Value.ToString());
		}

		public void WriteValue(string Name, double Value)
		{
			WriteValueInternal(Name, Value.ToString());
		}

		public void WriteValue(string Name, bool Value)
		{
			WriteValueInternal(Name, Value ? "true" : "false");
		}

		void WriteCommaNewline()
		{
			if (bRequiresComma)
			{
				Writer.WriteLine(",");
			}
			else if (Indent.Length > 0)
			{
				Writer.WriteLine();
			}
		}

		void WriteValueInternal(string Name, string Value)
		{
			WriteCommaNewline();

			Writer.Write("{0}\"{1}\" : {2}", Indent, Name, Value);

			bRequiresComma = true;
		}

		void WriteEscapedString(string Value)
		{
			// Escape any characters which may not appear in a JSON string (see http://www.json.org).
			Writer.Write("\"");
			if(Value != null)
			{
				for(int Idx = 0; Idx < Value.Length; Idx++)
				{
					switch(Value[Idx])
					{
						case '\"':
							Writer.Write("\\\"");
							break;
						case '\\':
							Writer.Write("\\\\");
							break;
						case '\b':
							Writer.Write("\\b");
							break;
						case '\f':
							Writer.Write("\\f");
							break;
						case '\n':
							Writer.Write("\\n");
							break;
						case '\r':
							Writer.Write("\\r");
							break;
						case '\t':
							Writer.Write("\\t");
							break;
						default:
							if(Char.IsControl(Value[Idx]))
							{
								Writer.Write("\\u{0:X4}", (int)Value[Idx]);
							}
							else
							{
								Writer.Write(Value[Idx]);
							}
							break;
					}
				}
			}
			Writer.Write("\"");
		}
	}
}
