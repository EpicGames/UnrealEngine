// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Implements similar functionality to TextWriter, but outputs an entire line at a time. Used as a base class for logging.
	/// </summary>
	abstract class LineBasedTextWriter
	{
		public abstract void WriteLine(string Text);

		public void WriteLine()
		{
			WriteLine("");
		}

		public void WriteLine(string Format, params object[] Args)
		{
			WriteLine(String.Format(Format, Args));
		}

		public void WriteWarning(FileReference File, string Message)
			=> WriteLine($"{File}: warning: {Message}");

		public void WriteWarning(FileReference File, int LineNumber, string Message)
			=> WriteLine($"{File}({LineNumber + 1}): warning: {Message}");

		public void WriteWarning(FileReference File, string Message, params object[] Args)
			=> WriteWarning(File, String.Format(Message, Args));

		public void WriteWarning(FileReference File, int LineNumber, string Message, params object[] Args)
			=> WriteWarning(File, LineNumber, String.Format(Message, Args));
	}
}
