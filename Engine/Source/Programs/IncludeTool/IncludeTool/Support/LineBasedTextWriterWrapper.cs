// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Implementation of LineBasedTextWriter which forwards data to a TextWriter instance.
	/// </summary>
	class LineBasedTextWriterWrapper : LineBasedTextWriter
	{
		TextWriter Inner;

		public LineBasedTextWriterWrapper(TextWriter Inner)
		{
			this.Inner = Inner;
		}

		public override void WriteLine(string Line)
		{
			Inner.WriteLine(Line);
		}
	}
}
