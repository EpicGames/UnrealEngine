// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Provides for writing log messages to a log file and the display, as well as setting status messages
	/// </summary>
	class LogWriter : LineBasedTextWriter, IDisposable
	{
		/// <summary>
		/// Output stream for log text
		/// </summary>
		StreamWriter FileWriter;

		/// <summary>
		/// Construct a log writer backed by the given file
		/// </summary>
		/// <param name="FileName">Filename to write to</param>
		public LogWriter(string FileName)
		{
			FileWriter = new StreamWriter(FileName);
			FileWriter.AutoFlush = true;
		}

		/// <summary>
		/// Dispose of the current object, and its resources
		/// </summary>
		public void Dispose()
		{
			if(FileWriter != null)
			{
				FileWriter.Dispose();
				FileWriter = null;
			}
		}

		/// <summary>
		/// Writes a character to the output stream
		/// </summary>
		/// <param name="Value">Character to write</param>
		public override void WriteLine(string Text)
		{
			FileWriter.WriteLine(Text);
			Console.WriteLine(Text);
		}
	}
}
