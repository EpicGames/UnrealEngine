// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Win32.SafeHandles;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Base class for a managed task, which may be executed as a thread or in a separate process
	/// </summary>
	abstract class ManagedTask : IDisposable
	{
		/// <summary>
		/// Event which is set when the task completes
		/// </summary>
		ManualResetEvent FinishedEvent;

		/// <summary>
		/// Output from the task
		/// </summary>
		BufferedTextWriter BufferedWriter = new BufferedTextWriter();

		/// <summary>
		/// Constructor
		/// </summary>
		protected ManagedTask()
		{
		}

		/// <summary>
		/// Disposes of the task
		/// </summary>
		public void Dispose()
		{
			Abort();
		}

		/// <summary>
		/// Start the task
		/// </summary>
		public void Start()
		{
			Debug.Assert(FinishedEvent == null);
			FinishedEvent = new ManualResetEvent(false);
			InternalStart(FinishedEvent, BufferedWriter);
		}

		/// <summary>
		/// When implemented by a derived class, implements the starting of the task
		/// </summary>
		/// <param name="FinishedEvent">Event to set once the task is comlete</param>
		/// <param name="Writer">Output writer for log events</param>
		protected abstract void InternalStart(ManualResetEvent FinishedEvent, LineBasedTextWriter Writer);

		/// <summary>
		/// Waits for the task to complete and returns
		/// </summary>
		/// <param name="LogOutput">Receives the output text from the task</param>
		/// <returns>The task's exit code</returns>
		public int Join(LineBasedTextWriter Writer)
		{
			FinishedEvent.WaitOne();

			int Result = InternalJoin();

			FinishedEvent.Dispose();
			FinishedEvent = null;

			BufferedWriter.CopyTo(Writer, "");
			BufferedWriter.Clear();

			return Result;
		}

		/// <summary>
		/// When implemented by a derived class, implements joining with the calling thread
		/// </summary>
		/// <returns>The task's exit code</returns>
		protected abstract int InternalJoin();

		/// <summary>
		/// Abort the task
		/// </summary>
		public void Abort()
		{
			if(FinishedEvent != null)
			{
				InternalAbort();

				BufferedWriter.Clear();

				FinishedEvent.Dispose();
				FinishedEvent = null;
			}
		}

		/// <summary>
		/// When implemented by a derived class, aborts the running task.
		/// </summary>
		protected abstract void InternalAbort();

		/// <summary>
		/// Waits for any of the given tasks to complete
		/// </summary>
		/// <param name="InTasks">The tasks to wait for</param>
		/// <param name="TimeoutMilliseconds">The maximum amount of time to wait before returning</param>
		/// <returns>The completed task index, or -1 if the timeout occurred</returns>
		public static int WaitForAny(ManagedTask[] Tasks, int TimeoutMilliseconds)
		{
			// There's a 64-handle limit for WaitHandle.WaitAny, but just use that if we can
			if(Tasks.Length <= 64)
			{
				return WaitHandle.WaitAny(Tasks.Select(x => x.FinishedEvent).ToArray());
			}

			// Otherwise cycle between 64-handle windows of the running task list until one of them finishes
			Stopwatch Timer = Stopwatch.StartNew();
			for(;;)
			{
				for(int BaseIndex = 0; BaseIndex < Tasks.Length; BaseIndex += 64)
				{
					int Timeout = (int)Math.Max(0, Math.Min(10, TimeoutMilliseconds - Timer.ElapsedMilliseconds));
					int Count = Math.Min(Tasks.Length - BaseIndex, 64);
					int Index = WaitHandle.WaitAny(Tasks.Skip(BaseIndex).Take(Count).Select(x => x.FinishedEvent).ToArray(), Timeout);
					if(Index >= 0 && Index < Count)
					{
						return BaseIndex + Index;
					}
				}
				if(Timer.ElapsedMilliseconds > TimeoutMilliseconds)
				{
					return -1;
				}
			}
		}
	}

	/// <summary>
	/// Implements a managed task using another thread
	/// </summary>
	class ManagedTaskThread : ManagedTask
	{
		/// <summary>
		/// The worker thread instance
		/// </summary>
		Thread Worker;

		/// <summary>
		/// The action to execute
		/// </summary>
		Func<LineBasedTextWriter, int> Action;

		/// <summary>
		/// Stores the result of executing the action
		/// </summary>
		int Result;

		/// <summary>
		/// Construct a managed task thread
		/// </summary>
		/// <param name="Action">The action to execute</param>
		public ManagedTaskThread(Func<LineBasedTextWriter, int> Action)
		{
			this.Action = Action;
		}

		/// <summary>
		/// Overridden from ManagedTask. Executes the given action.
		/// </summary>
		/// <param name="Event">Event to set when the action completes</param>
		/// <param name="Writer">Writer for log output</param>
		protected override void InternalStart(ManualResetEvent Event, LineBasedTextWriter Writer)
		{
			Worker = new Thread(x => ExecuteAction(Event, Writer));
			Worker.Start();
		}

		/// <summary>
		/// Overridden from ManagedTask. Waits for the task to complete.
		/// </summary>
		/// <returns>The task return code</returns>
		protected override int InternalJoin()
		{
			Worker.Join();
			Worker = null;
			return Result;
		}

		/// <summary>
		/// Aborts the running task
		/// </summary>
		protected override void InternalAbort()
		{
			// Thread.Abort() is obsolete, not supported, and throws PlatformNotSupportedException
			// Worker.Abort();

			// So, instead, we'll just wait for the worker to finish. If this turns out to take too long,
			// an explicit mechanism should be added to gracefully interrupt the worker thread sooner.
			Worker.Join();
			Worker = null;
		}

		/// <summary>
		/// Internal callback to execute the action
		/// </summary>
		/// <param name="Event">Event to set when the action completes</param>
		/// <param name="Writer">Writer for log output</param>
		void ExecuteAction(ManualResetEvent Event, LineBasedTextWriter Writer)
		{
			Result = Action(Writer);
			Event.Set();
		}
	}

	/// <summary>
	/// Implements a managed task using a separate process
	/// </summary>
	class ManagedTaskProcess : ManagedTask
	{
		/// <summary>
		/// Executable to run
		/// </summary>
		string FileName;

		/// <summary>
		/// Arguments for the child process
		/// </summary>
		string Arguments;

		/// <summary>
		/// The process object
		/// </summary>
		Process InnerProcess;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InFileName">The executable to run</param>
		/// <param name="InArguments">Arguments for the child process</param>
		public ManagedTaskProcess(string FileName, string Arguments)
		{
			this.FileName = FileName;
			this.Arguments = Arguments;
		}

		/// <summary>
		/// Overridden from ManagedTask. Executes the given action.
		/// </summary>
		/// <param name="Event">Event to set when the action completes</param>
		/// <param name="Writer">Writer for log output</param>
		protected override void InternalStart(ManualResetEvent Event, LineBasedTextWriter Writer)
		{
			DataReceivedEventHandler OutputHandler = (Sender, Args) => { if(!String.IsNullOrEmpty(Args.Data)) { Writer.WriteLine(Args.Data); } };

			InnerProcess = new Process();
			InnerProcess.StartInfo.FileName = FileName;
			InnerProcess.StartInfo.Arguments = Arguments;
			InnerProcess.StartInfo.UseShellExecute = false;
			InnerProcess.StartInfo.RedirectStandardOutput = true;
			InnerProcess.StartInfo.RedirectStandardError = true;
			InnerProcess.OutputDataReceived += OutputHandler;
			InnerProcess.ErrorDataReceived += OutputHandler;
			InnerProcess.Start();
			InnerProcess.BeginOutputReadLine();
			InnerProcess.BeginErrorReadLine();

			if(Environment.GetEnvironmentVariable("XoreaxBuildContext") != "1")
            {
				try { InnerProcess.PriorityClass = ProcessPriorityClass.BelowNormal; } catch (Exception) {  }
            }

			Event.SafeWaitHandle = new SafeWaitHandle(InnerProcess.Handle, false);
		}

		/// <summary>
		/// Overridden from ManagedTask. Waits for the task to complete.
		/// </summary>
		/// <returns>The task return code</returns>
		protected override int InternalJoin()
		{
			InnerProcess.WaitForExit();
			int ExitCode = InnerProcess.ExitCode;
			InnerProcess.Dispose();
			InnerProcess = null;
			return ExitCode;
		}

		/// <summary>
		/// Aborts the running task
		/// </summary>
		protected override void InternalAbort()
		{
			InnerProcess.Kill();
			InnerProcess.WaitForExit();
			InnerProcess.Dispose();
			InnerProcess = null;
		}
	}
}
