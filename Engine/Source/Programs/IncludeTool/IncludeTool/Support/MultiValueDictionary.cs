// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// Implements an unordered mapping from a key to a number of different values
	/// </summary>
	/// <typeparam name="TKey">The key type</typeparam>
	/// <typeparam name="TValue">The value type</typeparam>
	[Serializable]
	[DebuggerDisplay("Count = {Count}")]
	[DebuggerTypeProxy(typeof(MultiValueDictionary<,>.DebuggerProxy))]
	public class MultiValueDictionary<TKey, TValue> : IDictionary<TKey, IReadOnlyCollection<TValue>>, IReadOnlyDictionary<TKey, IReadOnlyCollection<TValue>>, IEnumerable<KeyValuePair<TKey, TValue>>
	{
		/// <summary>
		/// Helper class which adds the IReadOnlyCollection interface to HashSet
		/// </summary>
		class ValueSet : HashSet<TValue>, IReadOnlyCollection<TValue>
		{
			/// <summary>
			/// Default constructor
			/// </summary>
			public ValueSet(IEqualityComparer<TValue> ValueComparer) : base(ValueComparer)
			{
			}

			/// <summary>
			/// Construct a value set from a sequence of values
			/// </summary>
			/// <param name="Values">Values to initialize the set with</param>
			public ValueSet(IEnumerable<TValue> Values, IEqualityComparer<TValue> ValueComparer) : base(Values, ValueComparer)
			{
			}
		}

		/// <summary>
		/// Item to display in the debugger
		/// </summary>
		public class DebuggerProxyItem
		{
			/// <summary>
			/// The item key
			/// </summary>
			public TKey Key;

			/// <summary>
			/// The item values, displayed as an inline list
			/// </summary>
			[DebuggerBrowsable(DebuggerBrowsableState.RootHidden)]
			public TValue[] Values;

			/// <summary>
			/// Construct a proxy item
			/// </summary>
			public DebuggerProxyItem(TKey InKey, IEnumerable<TValue> InValues)
			{
				Key = InKey;
				Values = InValues.ToArray();
			}

			/// <summary>
			/// Get the summary string to display for this item
			/// </summary>
			/// <returns></returns>
			public override string ToString()
			{
				return String.Format("{0}, Count = {1}", Key, Values.Length);
			}
		}

		/// <summary>
		/// Helper class to allow the container to be visualized as an array
		/// </summary>
		class DebuggerProxy
		{
			/// <summary>
			/// Construct a proxy from a dictionary
			/// </summary>
			/// <param name="InDictionary">The underlying dictionary</param>
			public DebuggerProxy(IEnumerable<KeyValuePair<TKey, IReadOnlyCollection<TValue>>> Dictionary)
			{
				List<DebuggerProxyItem> ItemsList = new List<DebuggerProxyItem>();
				foreach(KeyValuePair<TKey, IReadOnlyCollection<TValue>> Pair in Dictionary)
				{
					ItemsList.Add(new DebuggerProxyItem(Pair.Key, Pair.Value));
				}
				Items = ItemsList.ToArray();
			}

			/// <summary>
			/// Return the items to display in the debugger.
			/// </summary>
			[DebuggerBrowsable(DebuggerBrowsableState.RootHidden)]
			public DebuggerProxyItem[] Items
			{
				get;
				set;
			}
		}

		/// <summary>
		/// The base mapping of key to set of values
		/// </summary>
		Dictionary<TKey, ValueSet> Contents;

		/// <summary>
		/// The default value comparer
		/// </summary>
		IEqualityComparer<TValue> ValueComparer;

		/// <summary>
		/// Default constructor
		/// </summary>
		public MultiValueDictionary()
		{
			Contents = new Dictionary<TKey, ValueSet>();
		}

		/// <summary>
		/// Construct a multi-value dictionary with the given key comparison class
		/// </summary>
		/// <param name="KeyComparer">Class to use to compare keys</param>
		public MultiValueDictionary(IEqualityComparer<TKey> KeyComparer)
		{
			Contents = new Dictionary<TKey, ValueSet>(KeyComparer);
		}

		/// <summary>
		/// Construct a multi-value dictionary with the given key and value comparison class
		/// </summary>
		/// <param name="KeyComparer">Class to use to compare keys</param>
		/// <param name="ValueComparer">Class to use to compare values</param>
		public MultiValueDictionary(IEqualityComparer<TKey> KeyComparer, IEqualityComparer<TValue> ValueComparer)
		{
			Contents = new Dictionary<TKey, ValueSet>(KeyComparer);
			this.ValueComparer = ValueComparer;
		}

		/// <summary>
		/// Always returns false.
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		public bool IsReadOnly
		{
			get { return false; }
		}

		/// <summary>
		/// The number of keys in the container.
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		public int Count
		{
			get { return Contents.Count; }
		}

		/// <summary>
		/// The unique keys in the container.
		/// </summary>
		public ICollection<TKey> Keys
		{
			get { return Contents.Keys; }
		}

		/// <summary>
		/// The unique keys in the container
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		IEnumerable<TKey> IReadOnlyDictionary<TKey, IReadOnlyCollection<TValue>>.Keys
		{
			get { return ((IReadOnlyDictionary<TKey, ValueSet>)Contents).Keys; }
		}

		/// <summary>
		/// The values in the dictionary.
		/// </summary>
		public ICollection<IReadOnlyCollection<TValue>> Values
		{
			get { return Contents.Values.ToArray(); }
		}

		/// <summary>
		/// The values in the container
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		IEnumerable<IReadOnlyCollection<TValue>> IReadOnlyDictionary<TKey, IReadOnlyCollection<TValue>>.Values
		{
			get { return ((IReadOnlyDictionary<TKey, ValueSet>)Contents).Values; }
		}

		/// <summary>
		/// Enumerates all the values matching the given key. Does not fail if the key doesn't exist.
		/// </summary>
		/// <param name="Key">The key to look up</param>
		/// <returns>Sequence of values for the given key</returns>
		public IEnumerable<TValue> WithKey(TKey Key)
		{
			ValueSet Values;
			if (Contents.TryGetValue(Key, out Values))
			{
				foreach (TValue Value in Values)
				{
					yield return Value;
				}
			}
		}

		/// <summary>
		/// Allows retrieval of a set of values for each key. Throws an exception if the key does not exist.
		/// </summary>
		/// <param name="Key">The key to look up</param>
		/// <returns>Collection of values for the given key</returns>
		public IReadOnlyCollection<TValue> this[TKey Key]
		{
			get { return Contents[Key]; }
			set { Contents[Key] = new ValueSet(value, ValueComparer); }
		}

		/// <summary>
		/// Clears the dictionary
		/// </summary>
		public void Clear()
		{
			Contents.Clear();
		}

		/// <summary>
		/// Adds a key with the given value
		/// </summary>
		/// <param name="Key">Key of the item to add</param>
		/// <param name="Value">Value to add</param>
		/// <returns>True if the item was added, false if it already exists</returns>
		public bool Add(TKey Key, TValue Value)
		{
			ValueSet Values;
			if (!Contents.TryGetValue(Key, out Values))
			{
				Values = new ValueSet(ValueComparer);
				Contents[Key] = Values;
			}
			return Values.Add(Value);
		}

		/// <summary>
		/// Adds a key with the given values
		/// </summary>
		/// <param name="Key">Key of the item to add</param>
		/// <param name="Values">Values to add</param>
		/// <returns>True if the item was added, false if it already exists</returns>
		public void Add(TKey Key, IEnumerable<TValue> Values)
		{
			Contents.Add(Key, new ValueSet(Values, ValueComparer));
		}

		/// <summary>
		/// Adds a key with the given values
		/// </summary>
		/// <param name="Key">Key of the item to add</param>
		/// <param name="Values">Values to add</param>
		/// <returns>True if the item was added, false if it already exists</returns>
		void IDictionary<TKey, IReadOnlyCollection<TValue>>.Add(TKey Key, IReadOnlyCollection<TValue> Values)
		{
			Contents.Add(Key, new ValueSet(Values, ValueComparer));
		}

		/// <summary>
		/// Adds a key/value pair
		/// </summary>
		/// <param name="Pair">Pair of items to add</param>
		void ICollection<KeyValuePair<TKey, IReadOnlyCollection<TValue>>>.Add(KeyValuePair<TKey, IReadOnlyCollection<TValue>> Pair)
		{
			Add(Pair.Key, Pair.Value);
		}

		/// <summary>
		/// Removes a key/value pair from the dictionary
		/// </summary>
		/// <param name="Key">Key of the item to remove</param>
		/// <param name="Value">Value of the item to remove</param>
		/// <returns>True if the item was removed, false if it did not exist</returns>
		public bool Remove(TKey Key, TValue Value)
		{
			ValueSet Values;
			if (Contents.TryGetValue(Key, out Values))
			{
				if (Values.Remove(Value))
				{
					if (Values.Count == 0)
					{
						Contents.Remove(Key);
					}
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Remove all values associated with a key
		/// </summary>
		/// <param name="Key">The key to remove</param>
		/// <returns>True if items with the given key were removed</returns>
		public bool Remove(TKey Key)
		{
			return Contents.Remove(Key);
		}

		/// <summary>
		/// Remove a given key/value pair from the collection
		/// </summary>
		/// <param name="Pair">The pair to remove</param>
		/// <returns>True if the pair was removed, false otherwise</returns>
		bool ICollection<KeyValuePair<TKey, IReadOnlyCollection<TValue>>>.Remove(KeyValuePair<TKey, IReadOnlyCollection<TValue>> Pair)
		{
			ValueSet ExistingValues;
			return Contents.TryGetValue(Pair.Key, out ExistingValues) && ExistingValues == Pair.Value;
		}

		/// <summary>
		/// Checks whether the given key/value pair exists in the container
		/// </summary>
		/// <param name="Key">Key of the item to look for</param>
		/// <param name="Value">Value of the item to look for</param>
		/// <returns>True if the item exists, false otherwise</returns>
		public bool Contains(TKey Key, TValue Value)
		{
			ValueSet Values;
			return Contents.TryGetValue(Key, out Values) && Values.Contains(Value);
		}

		/// <summary>
		/// Checks whether the given key/value pair exists in the container
		/// </summary>
		/// <param name="Pair">The key/value pair to look for</param>
		/// <returns>True if the collection contains the given pair, false otherwise</returns>
		bool ICollection<KeyValuePair<TKey, IReadOnlyCollection<TValue>>>.Contains(KeyValuePair<TKey, IReadOnlyCollection<TValue>> Pair)
		{
			ValueSet Values;
			return Contents.TryGetValue(Pair.Key, out Values) && Values == Pair.Value;
		}

		/// <summary>
		/// Checks whether the given key exists in the container
		/// </summary>
		/// <param name="Key">Key to look for</param>
		/// <returns>True if the key exists, false otherwise</returns>
		public bool ContainsKey(TKey Key)
		{
			return Contents.ContainsKey(Key);
		}

		/// <summary>
		/// Tries to get the collection of values for the given key
		/// </summary>
		/// <param name="Key">Key to look for</param>
		/// <param name="Values">Collection of matching values</param>
		/// <returns>True if the key was found (and Values is set to the result), false otherwise</returns>
		bool IDictionary<TKey, IReadOnlyCollection<TValue>>.TryGetValue(TKey Key, out IReadOnlyCollection<TValue> Values)
		{
			return TryGetValues(Key, out Values);
		}

		/// <summary>
		/// Tries to get the collection of values for the given key
		/// </summary>
		/// <param name="Key">Key to look for</param>
		/// <param name="Values">Collection of matching values</param>
		/// <returns>True if the key was found (and Values is set to the result), false otherwise</returns>
		bool IReadOnlyDictionary<TKey, IReadOnlyCollection<TValue>>.TryGetValue(TKey Key, out IReadOnlyCollection<TValue> Values)
		{
			return TryGetValues(Key, out Values);
		}

		/// <summary>
		/// Tries to get the collection of values for the given key
		/// </summary>
		/// <param name="Key">Key to look for</param>
		/// <param name="Values">Collection of matching values</param>
		/// <returns>True if the key was found (and Values is set to the result), false otherwise</returns>
		public bool TryGetValues(TKey Key, out IReadOnlyCollection<TValue> Values)
		{
			ValueSet RawValues;
			if (Contents.TryGetValue(Key, out RawValues))
			{
				Values = RawValues;
				return true;
			}
			else
			{
				Values = null;
				return false;
			}
		}

		/// <summary>
		/// Copy the key/value pairs to an array
		/// </summary>
		/// <param name="Array">The target array</param>
		/// <param name="ArrayIndex">The initial index to copy to</param>
		public void CopyTo(KeyValuePair<TKey, TValue>[] Array, int ArrayIndex)
		{
			foreach (KeyValuePair<TKey, ValueSet> Pair in Contents)
			{
				foreach(TValue Value in Pair.Value)
				{
					Array[ArrayIndex++] = new KeyValuePair<TKey, TValue>(Pair.Key, Value);
				}
			}
		}

		/// <summary>
		/// Copies the contents of this dictionary to an array of key/collection pairs
		/// </summary>
		/// <param name="Array">The target array</param>
		/// <param name="ArrayIndex">The initial index to copy to</param>
		void ICollection<KeyValuePair<TKey, IReadOnlyCollection<TValue>>>.CopyTo(KeyValuePair<TKey, IReadOnlyCollection<TValue>>[] Array, int ArrayIndex)
		{
			foreach (KeyValuePair<TKey, ValueSet> Pair in Contents)
			{
				Array[ArrayIndex++] = new KeyValuePair<TKey, IReadOnlyCollection<TValue>>(Pair.Key, Pair.Value);
			}
		}

		/// <summary>
		/// Gets an enumerator for the collection
		/// </summary>
		/// <returns>An enumerator object</returns>
		public IEnumerator<KeyValuePair<TKey, TValue>> GetEnumerator()
		{
			foreach (KeyValuePair<TKey, ValueSet> Pair in Contents)
			{
				foreach(TValue Value in Pair.Value)
				{
					yield return new KeyValuePair<TKey, TValue>(Pair.Key, Value);
				}
			}
		}

		/// <summary>
		/// Gets an enumerator for the collection
		/// </summary>
		/// <returns>An enumerator object</returns>
		IEnumerator<KeyValuePair<TKey, IReadOnlyCollection<TValue>>> IEnumerable<KeyValuePair<TKey, IReadOnlyCollection<TValue>>>.GetEnumerator()
		{
			foreach (KeyValuePair<TKey, ValueSet> Pair in Contents)
			{
				yield return new KeyValuePair<TKey, IReadOnlyCollection<TValue>>(Pair.Key, Pair.Value);
			}
		}

		/// <summary>
		/// Gets an enumerator for the collection
		/// </summary>
		/// <returns>An enumerator object</returns>
		IEnumerator IEnumerable.GetEnumerator()
		{
			return GetEnumerator();
		}
	}
}
