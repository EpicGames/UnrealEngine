// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool.Support
{
	/// <summary>
	/// The line/column of a character in a text file
	/// </summary>
	[Serializable]
	struct TextLocation
	{
		/// <summary>
		/// The zero-based line index
		/// </summary>
		public int LineIdx;

		/// <summary>
		/// The zero-based column index
		/// </summary>
		public int ColumnIdx;

		/// <summary>
		/// Property for the start of any text document
		/// </summary>
		public static TextLocation Origin
		{
			get { return new TextLocation(0, 0); }
		}

		/// <summary>
		/// Construct a text location from the given line and column index
		/// </summary>
		/// <param name="InLineIdx">The zero-based line index</param>
		/// <param name="InColumnIdx">The zero-based column index</param>
		public TextLocation(int InLineIdx, int InColumnIdx)
		{
			LineIdx = InLineIdx;
			ColumnIdx = InColumnIdx;
		}

		/// <summary>
		/// Gets a text location representing the start of next line
		/// </summary>
		/// <returns>TextLocation representing the start of the next line</returns>
		public TextLocation GetStartOfNextLine()
		{
			return new TextLocation(LineIdx + 1, 0);
		}

		/// <summary>
		/// Checks if one TextLocation object is before another
		/// </summary>
		/// <param name="A">First location to check</param>
		/// <param name="B">Second location to check</param>
		/// <returns>True if the first location is strictly before the second</returns>
		public static bool operator <(TextLocation A, TextLocation B)
		{
			return (A.LineIdx < B.LineIdx) || (A.LineIdx == B.LineIdx && A.ColumnIdx < B.ColumnIdx);
		}

		/// <summary>
		/// Checks if one TextLocation object is before or equal to another
		/// </summary>
		/// <param name="A">First location to check</param>
		/// <param name="B">Second location to check</param>
		/// <returns>True if the first location is before or equal to the second</returns>
		public static bool operator <=(TextLocation A, TextLocation B)
		{
			return (A.LineIdx < B.LineIdx) || (A.LineIdx == B.LineIdx && A.ColumnIdx <= B.ColumnIdx);
		}

		/// <summary>
		/// Checks if one TextLocation object is after another
		/// </summary>
		/// <param name="A">First location to check</param>
		/// <param name="B">Second location to check</param>
		/// <returns>True if the first location is strictly after the second</returns>
		public static bool operator >(TextLocation A, TextLocation B)
		{
			return (A.LineIdx > B.LineIdx) || (A.LineIdx == B.LineIdx && A.ColumnIdx > B.ColumnIdx);
		}

		/// <summary>
		/// Checks if one TextLocation object is after or equal to another
		/// </summary>
		/// <param name="A">First location to check</param>
		/// <param name="B">Second location to check</param>
		/// <returns>True if the first location is after or equal to the second</returns>
		public static bool operator >=(TextLocation A, TextLocation B)
		{
			return (A.LineIdx > B.LineIdx) || (A.LineIdx == B.LineIdx && A.ColumnIdx >= B.ColumnIdx);
		}

		/// <summary>
		/// Formats a TextLocation as a string as displayed in an IDE, in parentheses with line and column indices starting at one.
		/// </summary>
		/// <returns>True if the first location is strictly before the second</returns>
		public string GetSuffixForIDE()
		{
			return String.Format("({0},{1})", LineIdx + 1, ColumnIdx + 1);
		}

		/// <summary>
		/// Returns the zero-based line/column 
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return String.Format("{0}, {1}", LineIdx, ColumnIdx);
		}
	}

	/// <summary>
	/// Contains the contents of a text file, as an array of lines
	/// </summary>
	[Serializable]
	class TextBuffer
	{
		/// <summary>
		/// The lines in the file
		/// </summary>
		public readonly string[] Lines;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InLines">New array of lines for the text file</param>
		public TextBuffer(string[] InLines)
		{
			Lines = InLines;
		}

		/// <summary>
		/// Read a text file from a file
		/// </summary>
		/// <param name="InFileName">File name to read from</param>
		/// <returns>New instance of a text file object</returns>
		public static TextBuffer FromFile(string InFileName)
		{
			return new TextBuffer(File.ReadAllLines(InFileName));
		}

		/// <summary>
		/// Construct a text buffer from a string
		/// </summary>
		/// <param name="InFileName">File name to read from</param>
		/// <returns>New instance of a text buffer object</returns>
		public static TextBuffer FromString(string Text)
		{
			return new TextBuffer(Text.Split('\n'));
		}

		/// <summary>
		/// Property for the start of the text file
		/// </summary>
		public TextLocation Start
		{
			get { return TextLocation.Origin; }
		}

		/// <summary>
		/// Property for the end of the text file. This is one character beyond the last character in the file.
		/// </summary>
		public TextLocation End
		{
			get { return new TextLocation(Lines.Length, 0); }
		}

		/// <summary>
		/// Extract a portion of the file as a string
		/// </summary>
		/// <param name="BeginLineIdx">Index of the starting line</param>
		/// <param name="BeginColumnIdx">Index of the starting column</param>
		/// <param name="EndLineIdx">Index of the last line</param>
		/// <param name="EndColumnIdx">Index of the last column</param>
		/// <returns>String extracted from the file</returns>
		public string ExtractString(int BeginLineIdx, int BeginColumnIdx, int EndLineIdx, int EndColumnIdx)
		{
			if (BeginLineIdx == EndLineIdx)
			{
				if (EndLineIdx < Lines.Length)
				{
					return Lines[BeginLineIdx].Substring(BeginColumnIdx, EndColumnIdx - BeginColumnIdx);
				}
				else
				{
					return "";
				}
			}
			else
			{
				StringBuilder Result = new StringBuilder();
				Result.Append(Lines[BeginLineIdx], BeginColumnIdx, Lines[BeginLineIdx].Length - BeginColumnIdx);
				Result.AppendLine();
				for(int LineIdx = BeginLineIdx + 1; LineIdx < EndLineIdx; LineIdx++)
				{
					Result.AppendLine(Lines[LineIdx]);
				}
				if(EndLineIdx < Lines.Length)
				{
					Result.Append(Lines[EndLineIdx], 0, EndColumnIdx);
				}
				return Result.ToString();
			}
		}

		/// <summary>
		/// Extract a portion of the file as an array of strings
		/// </summary>
		/// <param name="BeginLineIdx">Index of the starting line</param>
		/// <param name="BeginColumnIdx">Index of the starting column</param>
		/// <param name="EndLineIdx">Index of the last line</param>
		/// <param name="EndColumnIdx">Index of the last column</param>
		/// <returns>Array of lines extracted from the file</returns>
		public string[] Extract(int BeginLineIdx, int BeginColumnIdx, int EndLineIdx, int EndColumnIdx)
		{
			string[] NewLines = new string[EndLineIdx - BeginLineIdx + 1];
			if (BeginLineIdx == EndLineIdx)
			{
				if (EndLineIdx < Lines.Length)
				{
					NewLines[0] = Lines[BeginLineIdx].Substring(BeginColumnIdx, EndColumnIdx - BeginColumnIdx);
				}
				else
				{
					NewLines[0] = "";
				}
			}
			else
			{
				NewLines[0] = Lines[BeginLineIdx].Substring(BeginColumnIdx);
				for (int Idx = 1; Idx < NewLines.Length - 1; Idx++)
				{
					NewLines[Idx] = Lines[BeginLineIdx + Idx];
				}

				if (EndLineIdx < Lines.Length)
				{
					NewLines[NewLines.Length - 1] = Lines[EndLineIdx].Substring(0, EndColumnIdx);
				}
				else
				{
					NewLines[NewLines.Length - 1] = "";
				}
			}
			return NewLines;
		}

		/// <summary>
		/// Extract a portion of the file as an array of strings
		/// </summary>
		/// <param name="Begin">Start of the range to extract</param>
		/// <param name="End">End of the range to extract, exclusive</param>
		/// <returns>Array of lines extracted from the file</returns>
		public string[] Extract(TextLocation Begin, TextLocation End)
		{
			return Extract(Begin.LineIdx, Begin.ColumnIdx, End.LineIdx, End.ColumnIdx);
		}

		/// <summary>
		/// Access a single line within the file
		/// </summary>
		/// <param name="LineIdx">The line  to return</param>
		/// <returns>The line at the given position</returns>
		public string this[int LineIdx]
		{
			get
			{
				if (LineIdx < 0 || LineIdx >= Lines.Length)
				{
					return "";
				}
				else
				{
					return Lines[LineIdx];
				}
			}
		}

		/// <summary>
		/// Access a single character within the file, by line and column
		/// </summary>
		/// <param name="LineIdx">The line of the character to return</param>
		/// <param name="ColumnIdx">The column index of the character to return</param>
		/// <returns>The character at the given position</returns>
		public char this[int LineIdx, int ColumnIdx]
		{
			get
			{
				if (LineIdx == Lines.Length)
				{
					return '\0';
				}
				else if (ColumnIdx == Lines[LineIdx].Length)
				{
					return '\n';
				}
				else
				{
					return Lines[LineIdx][ColumnIdx];
				}
			}
		}

		/// <summary>
		/// Access a single character within the file
		/// </summary>
		/// <param name="Location">Location of the character to return</param>
		/// <returns>The character at the given location</returns>
		public char this[TextLocation Location]
		{
			get { return this[Location.LineIdx, Location.ColumnIdx]; }
		}

		/// <summary>
		/// Reads a single character from a text buffer, and updates the current position
		/// </summary>
		/// <param name="LineIdx">Line index to read from</param>
		/// <param name="ColumnIdx">Column index to read from</param>
		/// <returns>The character at the given position</returns>
		public char ReadCharacter(ref int LineIdx, ref int ColumnIdx)
		{
			char Result = this[LineIdx, ColumnIdx];
			MoveNext(ref LineIdx, ref ColumnIdx);
			return Result;
		}

		/// <summary>
		/// Reads a single character from a text buffer, and updates the current position
		/// </summary>
		/// <param name="Location">Location to read from</param>
		/// <returns>The character at the given position</returns>
		public char ReadCharacter(ref TextLocation Location)
		{
			return ReadCharacter(ref Location.LineIdx, ref Location.ColumnIdx);
		}

		/// <summary>
		/// Updates a line and column index to contain the index of the next character
		/// </summary>
		/// <param name="LineIdx">The line index</param>
		/// <param name="ColumnIdx">The column index</param>
		public bool MoveNext(ref int LineIdx, ref int ColumnIdx)
		{
			if (LineIdx >= Lines.Length)
			{
				return false;
			}

			ColumnIdx++;
			if (ColumnIdx > Lines[LineIdx].Length)
			{
				ColumnIdx = 0;
				LineIdx++;
			}
			return true;
		}

		/// <summary>
		/// Updates a location to point to the next character
		/// </summary>
		/// <param name="Location">The location to update</param>
		public void MoveNext(ref TextLocation Location)
		{
			MoveNext(ref Location.LineIdx, ref Location.ColumnIdx);
		}

		/// <summary>
		/// Converts the text file to a string for debugging
		/// </summary>
		/// <returns>Concatenated string representation of the document</returns>
		public override string ToString()
		{
			return String.Join("\n", Lines);
		}
	}
}
