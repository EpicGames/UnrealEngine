// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Security.Cryptography;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using System.Text.RegularExpressions;

namespace IncludeTool.Support
{
	static class Utility
	{
		/// <summary>
		/// Tries to find the given program executable in one of the PATH directories
		/// </summary>
		/// <param name="ProgramExe">The program to search for</param>
		/// <param name="Location">On success, the location of the program. Null otherwise.</param>
		/// <returns>True if the program is found</returns>
		public static bool TryFindProgramInPath(string ProgramExe, out FileReference Location)
		{
			// Search through all the paths in the PATH environment variable
			foreach(string SearchPath in Environment.GetEnvironmentVariable("PATH").Split(Path.PathSeparator))
			{
				FileReference CandidateLocation = FileReference.Combine(new DirectoryReference(SearchPath), "p4.exe");
				if(FileReference.Exists(CandidateLocation))
				{
					Location = CandidateLocation;
					return true;
				}
			}

			// Return failure
			Location = null;
			return false;
		}

		/// <summary>
		/// Quotes an argument for passing on the command line, if necessary.
		/// </summary>
		/// <param name="Argument">The input argument</param>
		/// <returns>The quoted argument, or the original argument if it doesn't contain spaces</returns>
		public static string QuoteArgument(string Argument)
		{
			return Argument.Contains(' ') ? ("\"" + Argument + "\"") : Argument;
		}

		/// <summary>
		/// Formats a SHA digest as a string
		/// </summary>
		/// <param name="Hash">The SHA hash</param>
		/// <returns>String representation of the hash</returns>
		public static string FormatSHA(byte[] Hash)
		{
			StringBuilder Result = new StringBuilder();
			for(int Idx = 0; Idx < Hash.Length; Idx++)
			{
				Result.AppendFormat("{0:x2}", (int)Hash[Idx]);
			}
			return Result.ToString();
		}

		/// <summary>
		/// Computes the SHA1 digest for the given text
		/// </summary>
		/// <param name="Text">The text to compute a digest for</param>
		/// <returns>The digest, as a hex string</returns>
		public static string ComputeDigest(string Text)
		{
			byte[] Data = Encoding.Unicode.GetBytes(Text);
			return FormatSHA(SHA1.Create().ComputeHash(Data));
		}

		/// <summary>
		/// Computes the SHA1 digest for the file at the given location
		/// </summary>
		/// <param name="FileLocation">The file to compute a digest for</param>
		/// <returns>The digest, as a hex string</returns>
		public static string ComputeDigest(FileReference FileLocation)
		{
			byte[] Data = File.ReadAllBytes(FileLocation.FullName);
			return FormatSHA(SHA1.Create().ComputeHash(Data));
		}

		/// <summary>
		/// Execute an external process, writing its output to the log.
		/// </summary>
		/// <param name="Executable">Path to the executable to run</param>
		/// <param name="Arguments">Arguments for the program</param>
		/// <param name="WorkingDir">The working directory to run in</param>
		/// <param name="Log">Writer to receive output messages</param>
		public static int Run(FileReference ExecutableFile, string Arguments, DirectoryReference WorkingDir, LineBasedTextWriter Log)
		{
			using (Process NewProcess = new Process())
			{
				DataReceivedEventHandler EventHandler = (Sender, Args) => { if (!String.IsNullOrEmpty(Args.Data)) { Log.WriteLine(Args.Data); } };

				NewProcess.StartInfo.FileName = ExecutableFile.FullName;
				NewProcess.StartInfo.Arguments = Arguments;
				NewProcess.StartInfo.UseShellExecute = false;
				NewProcess.StartInfo.WorkingDirectory = WorkingDir.FullName;
				NewProcess.StartInfo.RedirectStandardOutput = true;
				NewProcess.StartInfo.RedirectStandardError = true;
				NewProcess.OutputDataReceived += EventHandler;
				NewProcess.ErrorDataReceived += EventHandler;

				NewProcess.Start();
				NewProcess.BeginOutputReadLine();
				NewProcess.BeginErrorReadLine();
				NewProcess.WaitForExit();

				return NewProcess.ExitCode;
			}
		}

		/// <summary>
		/// Copies a portion of the file to the given text writer
		/// </summary>
		/// <param name="Text">The text buffer to copy from</param>
		/// <param name="Location">The initial location</param>
		/// <param name="EndLocation">The final location</param>
		/// <param name="Writer">The writer to receive the excerpt</param>
		public static void CopyExcerpt(TextBuffer Text, TextLocation Location, TextLocation EndLocation, TextWriter Writer)
		{
			string[] TextLines = Text.Extract(Location, EndLocation);
			for (int Idx = 0; Idx < TextLines.Length - 1; Idx++)
			{
				Writer.WriteLine(TextLines[Idx]);
			}
			Writer.Write(TextLines[TextLines.Length - 1]);
		}

		/// <summary>
		/// Copies an excerpt from a file to a TextWriter, commenting each output line
		/// </summary>
		/// <param name="Text">The text buffer to copy from</param>
		/// <param name="Location">The initial location</param>
		/// <param name="EndLocation">The final location</param>
		/// <param name="Writer">The writer to receive the excerpt</param>
		public static void CopyCommentedExcerpt(TextWriter Writer, TextBuffer Text, TextLocation Location, TextLocation EndLocation)
		{
			string[] CommentLines = Text.Extract(Location, EndLocation);
			for (int Idx = 0; Idx < CommentLines.Length - 1; Idx++)
			{
				string CommentLine = CommentLines[Idx];
				if (CommentLine.EndsWith("\\")) CommentLine = CommentLine.Substring(CommentLine.Length - 1);
				Writer.WriteLine("// {0}", CommentLine);
			}
		}

		/// <summary>
		/// Remove relative path separators from a string, without normalizing slashes
		/// </summary>
		/// <param name="FileName">Path to normalize</param>
		/// <returns>Filename with relative paths removed, where possible</returns>
		public static string RemoveRelativePaths(string FileName)
		{
			for(int Idx = 0; Idx < FileName.Length; Idx++)
			{
				if(FileName[Idx] == '/' || FileName[Idx] == '\\')
				{
					if(Idx + 2 < FileName.Length && FileName[Idx + 1] == '.' && (FileName[Idx + 2] == '/' || FileName[Idx + 2] == '\\'))
					{
						// Remove the current directory fragment
						FileName = FileName.Substring(0, Idx) + FileName.Substring(Idx + 2);
						Idx--;
					}
					else if(Idx + 3 < FileName.Length && FileName[Idx + 1] == '.' && FileName[Idx + 2] == '.' && (FileName[Idx + 3] == '/' || FileName[Idx + 3] == '\\'))
					{
						// Scan back to the last separator
						for(int StartIdx = Idx - 1; StartIdx >= 0; StartIdx--)
						{
							if(FileName[StartIdx] == '/' || FileName[StartIdx] == '\\')
							{
								FileName = FileName.Substring(0, StartIdx) + FileName.Substring(Idx + 3);
								Idx = StartIdx - 1;
								break;
							}
						}
					}
				}
			}
			return FileName;
		}

		/// <summary>
		/// Helper class for maintaining the state of a parallel for loop, and outputting status update messages
		/// </summary>
		class ParallelForState
		{
			/// <summary>
			/// The number of completed iterations
			/// </summary>
			public int CompletedCount;

			/// <summary>
			/// Timer for outputting messages
			/// </summary>
			public Stopwatch Timer;

			/// <summary>
			/// The last UTC time that a status message was output
			/// </summary>
			public TimeSpan NextUpdateTime;

			/// <summary>
			/// Constructor
			/// </summary>
			public ParallelForState()
			{
				Timer = Stopwatch.StartNew();
				NextUpdateTime = TimeSpan.Zero;
			}

			/// <summary>
			/// Increment the number of iterations, and output a status update message if 5 seconds have elapsed
			/// </summary>
			/// <param name="TotalCount">The total number of iterations</param>
			/// <param name="Message">The message to output</param>
			/// <param name="Log">Writer for output messages</param>
			public void Increment(int TotalCount, string Message, LineBasedTextWriter Log)
			{
				lock(this)
				{
					CompletedCount++;

					if(Timer.Elapsed > NextUpdateTime && CompletedCount < TotalCount)
					{
						lock(Log)
						{
							Log.WriteLine("[{0}] {1} ({2}/{3}; {4}%)", Timer.Elapsed.ToString(@"hh\:mm\:ss"), Message, CompletedCount, TotalCount, (CompletedCount * 100) / TotalCount);
						}
						NextUpdateTime = Timer.Elapsed + TimeSpan.FromSeconds(5.0);
					}
				}
			}

			/// <summary>
			/// Print the last message showing 100%
			/// </summary>
			/// <param name="TotalCount">The total number of iterations</param>
			/// <param name="Message">The message to output</param>
			/// <param name="Log">Writer for output messages</param>
			public void Complete(int TotalCount, string Message, LineBasedTextWriter Log)
			{
				Log.WriteLine("[{0}] {1} ({2}/{3}; {4}%)", Timer.Elapsed.ToString(@"hh\:mm\:ss"), Message, TotalCount, TotalCount, 100);
			}
		}

		/// <summary>
		/// Execute a Parallel.For loop, but output status messages showing progress every 5 seconds
		/// </summary>
		/// <param name="Message">The message to output</param>
		/// <param name="BeginValue">The lower bound for the for loop, inclusive</param>
		/// <param name="EndValue">The upper bound for the for loop, exclusive</param>
		/// <param name="Action">The action to execute for each iteration</param>
		/// <param name="Log">Log for output messages</param>
		public static void ParallelForWithStatus(string Message, int BeginValue, int EndValue, Action<int> Action, LineBasedTextWriter Log)
		{
			ParallelForWithStatus(Message, BeginValue, EndValue, new ParallelOptions(), Action, Log);
		}

		/// <summary>
		/// Execute a Parallel.For loop, but output status messages showing progress every 5 seconds
		/// </summary>
		/// <param name="Message">The message to output</param>
		/// <param name="BeginValue">The lower bound for the for loop, inclusive</param>
		/// <param name="EndValue">The upper bound for the for loop, exclusive</param>
		/// <param name="Action">The action to execute for each iteration</param>
		/// <param name="Log">Log for output messages</param>
		public static void ParallelForWithStatus(string Message, int BeginValue, int EndValue, ParallelOptions Options, Action<int> Action, LineBasedTextWriter Log)
		{
			Log.WriteLine(Message);

			ParallelForState State = new ParallelForState();
			Parallel.For(BeginValue, EndValue, Options, Index => { Action(Index); State.Increment(EndValue - BeginValue, Message, Log); });

			State.Complete(EndValue - BeginValue, Message, Log);
		}

		/// <summary>
		/// Queries compiler for the version and returns string like "3.9.0" or such.
		/// </summary>
		private static string DetermineClangVersionDir(string ClangExecutable)
		{
			string CompilerVersionString = null;

			Process Proc = new Process();
			Proc.StartInfo.UseShellExecute = false;
			Proc.StartInfo.CreateNoWindow = true;
			Proc.StartInfo.RedirectStandardOutput = true;
			Proc.StartInfo.RedirectStandardError = true;

			Proc.StartInfo.FileName = ClangExecutable;
			Proc.StartInfo.Arguments = " --version";

			Proc.Start();
			Proc.WaitForExit();

			if (Proc.ExitCode == 0)
			{
				// read just the first string
				string VersionString = Proc.StandardOutput.ReadLine();

				Regex VersionPattern = new Regex("version \\d+(\\.\\d+)+");
				Match VersionMatch = VersionPattern.Match(VersionString);

				// version match will be like "version 3.3", so remove the "version"
				if (VersionMatch.Value.StartsWith("version "))
				{
					CompilerVersionString = VersionMatch.Value.Replace("version ", "");
				}
			}

			return CompilerVersionString;
		}

		/// <summary>
		/// Gets include directories that the compiler knows about.
		/// </summary>
		public static void GetSystemIncludeDirectories(CompileEnvironment CompileEnv, DirectoryReference InputDir, List<DirectoryReference> ExtraSystemIncludePaths)
		{
			// FIXME: assumes clang now, and assumes certain hard-coded dirs. Can be improved by querying the compiler instead.

			DirectoryReference SysRootDir = CompileEnv.Options.Where(x => x.Name.StartsWith("--sysroot=")).Select(x => new DirectoryReference(x.Name.Substring(10))).First();

			string CompilerFullPath = CompileEnv.Compiler.FullName;

			ExtraSystemIncludePaths.Add(DirectoryReference.Combine(SysRootDir, "usr", "include"));
			string CompilerVersionString = DetermineClangVersionDir(CompilerFullPath);
			if (!String.IsNullOrEmpty(CompilerVersionString))
			{
				// starting with 16.x clang includes are in a directory named only after its major version
				int Major = System.Version.Parse(CompilerVersionString).Major;
				if (Major >= 16)
				{
					CompilerVersionString = System.Version.Parse(CompilerVersionString).Major.ToString();
				}
				ExtraSystemIncludePaths.Add(DirectoryReference.Combine(SysRootDir, "lib", "clang", CompilerVersionString, "include"));
			}
			ExtraSystemIncludePaths.Add(DirectoryReference.Combine(InputDir, "Engine", "Source", "ThirdParty", "Linux", "LibCxx", "include", "c++", "v1"));
		}
	}
}
