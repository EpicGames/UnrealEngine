// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using IncludeTool.Support;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Type of a defined symbol
	/// </summary>
	enum SymbolType
	{
		/// <summary>
		/// A macro definition
		/// </summary>
		Macro,

		/// <summary>
		/// Class declaration
		/// </summary>
		Class,

		/// <summary>
		/// Struct declaration
		/// </summary>
		Struct,

		/// <summary>
		/// Enum declaration
		/// </summary>
		Enumeration,

		/// <summary>
		/// A typedef declaration
		/// </summary>
		Typedef,

		/// <summary>
		/// A template class declaration
		/// </summary>
		TemplateClass,

		/// <summary>
		/// A template struct declaration
		/// </summary>
		TemplateStruct,
	}

	/// <summary>
	/// Represents a symbol name, scraped from a source fragment
	/// </summary>
	[Serializable]
	class Symbol
	{
		/// <summary>
		/// The symbol name
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// Type of the symbol
		/// </summary>
		public readonly SymbolType Type;

		/// <summary>
		/// Text for a forward declaration of this symbol
		/// </summary>
		public readonly string ForwardDeclaration;

		/// <summary>
		/// Fragment that this symbol is defined in
		/// </summary>
		public readonly SourceFragment Fragment;

		/// <summary>
		/// The location within the file that this symbol is declared
		/// </summary>
		public readonly TextLocation Location;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Name">Symbol name</param>
		/// <param name="Type">Type of the symbol</param>
		/// <param name="ForwardDeclaration">Text for a forward declaration of this symbol</param>
		/// <param name="Fragment">File that this symbol is defined in</param>
		/// <param name="Location">Location within the file that this symbol is declared</param>
		public Symbol(string Name, SymbolType Type, string ForwardDeclaration, SourceFragment Fragment, TextLocation Location)
		{
			this.Name = Name;
			this.Type = Type;
			this.ForwardDeclaration = ForwardDeclaration;
			this.Fragment = Fragment;
			this.Location = Location;
		}

		/// <summary>
		/// Returns a string representation of the symbol for debugging purposes
		/// </summary>
		/// <returns>String containing the symbol name and type</returns>
		public override string ToString()
		{
			return String.Format("{0}: {1}", Type.ToString(), Name);
		}
	}

	/// <summary>
	/// The type of a symbol reference
	/// </summary>
	enum SymbolReferenceType
	{
		/// <summary>
		/// The symbol is only referenced in an opaque manner (ie. via a pointer or reference), and can be forward-declared.
		/// </summary>
		Opaque,

		/// <summary>
		/// The symbol is referenced in some other way
		/// </summary>
		RequiresDefinition
	}

	class UnbalancedScopeException : Exception
	{
		public UnbalancedScopeException(string Message) 
			: base(Message)
		{
		}
	}

	/// <summary>
	/// Registry of symbol names to definitions
	/// </summary>
	class SymbolTable
	{
		/// <summary>
		/// Mapping of name to symbol definition
		/// </summary>
		MultiValueDictionary<string, Symbol> Lookup = new MultiValueDictionary<string,Symbol>();

		/// <summary>
		/// Add all the exported symbols from the given source file
		/// </summary>
		/// <param name="File">The source fragment to parse</param>
		public void AddExports(SourceFile File)
		{
			if((File.Flags & SourceFileFlags.External) == 0 && (File.Flags & SourceFileFlags.IgnoreExportedSymbols) == 0)
			{
				TextBuffer Text = File.Text;
				if(Text != null)
				{
					int MarkupIdx = 0;
					int Scope = 0;
					ReadExportedSymbols(File, ref MarkupIdx, ref Scope);
				}
			}
		}

		/// <summary>
		/// Prints all the symbols with conflicting definitions
		/// </summary>
		/// <param name="InputDir"></param>
		/// <param name="Log">Writer for log messagsThe source fragment to parse</param>
		public void PrintConflicts(DirectoryReference InputDir, LineBasedTextWriter Log)
		{
			foreach(string SymbolName in Lookup.Keys)
			{
				IReadOnlyCollection<Symbol> Symbols = Lookup[SymbolName];
				if(Symbols.Count > 0)
				{
					int NumTypes = Symbols.Select(x => x.Type).Where(x => x != SymbolType.Macro).Distinct().Count();
					if(NumTypes > 1)
					{
						// UE-150382 workaround
						// It appears that symbols are being added to Lookup whether or not branches have been taken
						string FileName = Symbols.First().Fragment.File.Location.GetFileName();
						if (Rules.ReportConflictingSymbolsForFile(Symbols.First().Fragment.File.Location))
						{
						Log.WriteLine("warning: conflicting declarations of '{0}':", SymbolName);
							foreach (Symbol Symbol in Symbols)
						{
							string RelativePath = Symbol.Fragment.File.Location.MakeRelativeTo(InputDir);
								Log.WriteLine($"  {Symbol.Type} at {RelativePath}:{Symbol.Location.LineIdx + 1}");
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Read all the exported symbol definitions from a file
		/// </summary>
		/// <param name="File">The file to parse</param>
		/// <param name="MarkupIdx">The current index into the markup array of this file</param>
		/// <param name="Scope">The scoping depth, ie. number of unmatched '{' tokens encountered in source code</param>
		void ReadExportedSymbols(SourceFile File, ref int MarkupIdx, ref int Scope)
		{
			// Ignore files which only have a header guard; there are no fragments that we can link to
			if(File.BodyMinIdx == File.BodyMaxIdx)
			{
				return;
			}

			// Read markup from the file
			for(; MarkupIdx < File.Markup.Length; MarkupIdx++)
			{
				PreprocessorMarkup Markup = File.Markup[MarkupIdx];
				if (Markup.Type == PreprocessorMarkupType.Define)
				{
					// If the macro is not already defined, add it
					if (Markup.Tokens[0].Text != "LOCTEXT_NAMESPACE")
					{
						SourceFragment Fragment = FindFragment(File, MarkupIdx);
						AddSymbol(Markup.Tokens[0].Text, SymbolType.Macro, null, Fragment, Markup.Location);
					}
				}
				else if (Markup.Type == PreprocessorMarkupType.If && Markup.Tokens.Count == 2 && Markup.Tokens[0].Text == "!" && Markup.Tokens[1].Text == "CPP")
				{
					// Completely ignore these blocks; they sometimes differ in class/struct usage, and exist solely for noexport types to be parsed by UHT
					int Depth = 1;
					while(Depth > 0)
					{
						Markup = File.Markup[++MarkupIdx];
						if(Markup.Type == PreprocessorMarkupType.If || Markup.Type == PreprocessorMarkupType.Ifdef || Markup.Type == PreprocessorMarkupType.Ifndef)
						{
							Depth++;
						}
						else if(Markup.Type == PreprocessorMarkupType.Endif)
						{
							Depth--;
						}
					}
				}
				else if (Markup.Type == PreprocessorMarkupType.If || Markup.Type == PreprocessorMarkupType.Ifdef || Markup.Type == PreprocessorMarkupType.Ifndef)
				{
					// Loop through all the branches of this conditional block, saving the resulting scope of each derivation
					List<int> NewScopes = new List<int>();
					while (Markup.Type != PreprocessorMarkupType.Endif)
					{
						// Skip over the conditional directive
						MarkupIdx++;

						// Read any exported symbols from this conditional block
						int NewScope = Scope;
						ReadExportedSymbols(File, ref MarkupIdx, ref NewScope);
						NewScopes.Add(NewScope);

						// Get the new preprocessor markup
						Markup = File.Markup[MarkupIdx];
					}

					// Make sure they were all consistent
					if(NewScopes.Any(x => x != NewScopes[0]))
					{
						throw new UnbalancedScopeException(String.Format("Unbalanced scope depth between conditional block derivations while parsing {0} at pos {1}:{2} with end pos {3}:{4}",
                            File.Location,
                            Markup.Location.LineIdx + 1, Markup.Location.ColumnIdx,
                            Markup.EndLocation.LineIdx + 1, Markup.EndLocation.ColumnIdx));
					}

					// Update the scope
					Scope = NewScopes[0];
				}
				else if(Markup.Type == PreprocessorMarkupType.Else || Markup.Type == PreprocessorMarkupType.Elif || Markup.Type == PreprocessorMarkupType.Endif)
				{
					// We've reached the end of this block; return success
					break;
				}
				else if(Markup.Type == PreprocessorMarkupType.Text)
				{
					// Read the declarations in this block
					SourceFragment Fragment = FindFragment(File, MarkupIdx);
					ParseDeclarations(File, Fragment, File.Markup[MarkupIdx], ref Scope);
				}
			}
		}

		/// <summary>
		/// Parse declarations from the reader
		/// </summary>
		/// <param name="File">The file containing the declarations</param>
		/// <param name="MarkupIdx">The markup to parse declarations from</param>
		/// <param name="Scope">The scope depth; the number of unmatched opening brace tokens</param>
		void ParseDeclarations(SourceFile File, SourceFragment Fragment, PreprocessorMarkup Markup, ref int Scope)
		{
			TokenReader Reader = new TokenReader(File.Location.FullName, File.Text, Markup.Location, Markup.EndLocation);
			if(Reader.MoveNext())
			{
				for(;;)
				{
					if(Scope > 0 || (!ReadEnumClassHeader(Reader, Fragment) && !ReadClassOrStructHeader(Reader, Fragment) && !ReadTypedefHeader(Reader, Fragment) && !ReadTemplateClassOrStructHeader(Reader, Fragment) && !SkipTemplateHeader(Reader)))
					{
						// Update the scope depth
						if(Reader.Current.Text == "{")
						{
							Scope++;
						}
						else if(Reader.Current.Text == "}" && --Scope < 0)
						{
							throw new UnbalancedScopeException(String.Format("Unbalanced '}}' while parsing '{0}' at pos {1}:{2} with end pos {3}:{4}",
                                File.Location,
                                Reader.TokenLocation.LineIdx+1, Reader.TokenLocation.ColumnIdx,
                                Reader.TokenEndLocation.LineIdx + 1, Reader.TokenEndLocation.ColumnIdx));
						}

						// Move to the next token
						if(!Reader.MoveNext())
						{
							break;
						}
					}
				}
			}
		}

		/// <summary>
		/// Parse an "enum class" declaration, and add a symbol for it
		/// </summary>
		/// <param name="Reader">Tokens to be parsed. On success, this is assigned to a new </param>
		/// <param name="Fragment">Fragment containing these tokens</param>
		bool ReadEnumClassHeader(TokenReader OriginalReader, SourceFragment Fragment)
		{
			TokenReader Reader = new TokenReader(OriginalReader);

			// Read the UENUM prefix if present. We don't want to forward-declare types that need to be parsed by UHT, because it needs the definition.
			bool bIsUENUM = false;
			if (Reader.Current.Text == "UENUM")
			{
				if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines) || Reader.Current.Text != "(")
				{
					return false;
				}
				while(Reader.Current.Text != ")")
				{
					if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
					{
						return false;
					}
				}
				if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
				{
					return false;
				}
				bIsUENUM = true;
			}

			// Read the 'enum class' tokens
			if(Reader.Current.Text != "enum" || !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}
			if(Reader.Current.Text != "class" || !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}

			// Read the name, make sure we haven't read a definition for it already, and check it's an enum declaration
			string Name = Reader.Current.Text;
			if(Reader.Current.Type != TokenType.Identifier || !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}
			if(Reader.Current.Text != ";" && Reader.Current.Text != ":" && Reader.Current.Text != "{")
			{
				return false;
			}

			// Build the forward declaration for it. Don't forward-declare UENUM types because UHT needs to parse their definition first.
			string ForwardDeclaration = null;
			if(!bIsUENUM)
			{
				StringBuilder ForwardDeclarationBuilder = new StringBuilder();
				ForwardDeclarationBuilder.AppendFormat("enum class {0}", Name);
				while(Reader.Current.Text != ";" && Reader.Current.Text != "{")
				{
					// Append the next token
					if(Reader.Current.HasLeadingSpace)
					{
						ForwardDeclarationBuilder.Append(" ");
					}
					ForwardDeclarationBuilder.Append(Reader.Current.Text);

					// Try to move to the next token
					if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
					{
						return false;
					}
				}
				ForwardDeclarationBuilder.Append(";");
				ForwardDeclaration = ForwardDeclarationBuilder.ToString();
			}

			// Create a symbol for it if it's an actual definition rather than a forward declaration
			if(Reader.Current.Text == "{" && Rules.AllowSymbol(Name))
			{
				AddSymbol(Name, SymbolType.Enumeration, ForwardDeclaration, Fragment, OriginalReader.TokenLocation);
			}

			// Update the original reader to be the new location
			OriginalReader.Set(Reader);
			return true;
		}

		/// <summary>
		/// Try to read a class or struct header
		/// </summary>
		/// <param name="Reader">Tokens to parse</param>
		/// <param name="Fragment">Fragment containing these tokens</param>
		bool ReadClassOrStructHeader(TokenReader OriginalReader, SourceFragment Fragment)
		{
			TokenReader Reader = new TokenReader(OriginalReader);

			// Make sure it's the right type
			SymbolType Type;
			if(Reader.Current.Text == "class")
			{
				Type = SymbolType.Class;
			}
			else if(Reader.Current.Text == "struct")
			{
				Type = SymbolType.Struct;
			}
			else
			{
				return false;
			}

			// Move to the name
			string ClassOrStructKeyword = Reader.Current.Text;
			if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}

			// Skip over an optional DLL export declaration
			if(Reader.Current.Type == TokenType.Identifier && Reader.Current.Text.EndsWith("_API") && !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}

			// Read the class name
			string Name = Reader.Current.Text;
			if(Reader.Current.Type != TokenType.Identifier || !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}
			if(Reader.Current.Text != ":" && Reader.Current.Text != "{")
			{
				return false;
			}

			// Create the symbol
            if(Rules.AllowSymbol(Name))
            {
			    string ForwardDeclaration = String.Format("{0} {1};", ClassOrStructKeyword, Name);
			    AddSymbol(Name, Type, ForwardDeclaration, Fragment, OriginalReader.TokenLocation);
            }

			// Move the original reader past the declaration
			OriginalReader.Set(Reader);
			return true;
		}

		/// <summary>
		/// Try to skip over a template header
		/// </summary>
		/// <param name="OriginalReader">Tokens to parse</param>
		/// <param name="Fragment">Fragment containing these tokens</param>
		bool ReadTypedefHeader(TokenReader OriginalReader, SourceFragment Fragment)
		{
			TokenReader Reader = new TokenReader(OriginalReader);
			if(Reader.Current.Text == "typedef")
			{
				// Check for the typedef keyword
				Token PreviousToken = Reader.Current;
				if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
				{
					return false;
				}

				// Scan to the next semicolon
				while(Reader.MoveNext(TokenReaderContext.IgnoreNewlines) && Reader.Current.Text != ";" && Reader.Current.Text != "{")
				{
					PreviousToken = Reader.Current;
				}

				// Ignore 'typedef struct' and 'typedef union' declarations.
				if(Reader.Current.Text == "{")
				{
					return false;
				}

				// Try to add a symbol for the previous token. If it already exists, replace it. 
				if(PreviousToken.Type == TokenType.Identifier && Rules.AllowSymbol(PreviousToken.Text))
				{
					AddSymbol(PreviousToken.Text, SymbolType.Typedef, null, Fragment, OriginalReader.TokenLocation);
				}
			}
			else if(Reader.Current.Text == "using")
			{
				// Check for the using keyword
				if (!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
				{
					return false;
				}

				// Get the identifier
				Token Identifier = Reader.Current;
				if (Reader.Current.Type != TokenType.Identifier || !Reader.MoveNext(TokenReaderContext.IgnoreNewlines) || Reader.Current.Text != "=")
				{
					return false;
				}

				// Scan to the next semicolon
				while (Reader.MoveNext(TokenReaderContext.IgnoreNewlines) && Reader.Current.Text != ";" && Reader.Current.Text != "{")
				{
				}

				// Get the identifier
				if (Rules.AllowSymbol(Identifier.Text))
				{
					AddSymbol(Identifier.Text, SymbolType.Typedef, null, Fragment, OriginalReader.TokenLocation);
				}
			}
			else
			{
				return false;
			}

			// Move the original reader past the declaration
			OriginalReader.Set(Reader);
			return true;
		}

		/// <summary>
		/// Try to read a template class or struct header
		/// </summary>
		/// <param name="Reader">Tokens to parse</param>
		/// <param name="Fragment">Fragment containing these tokens</param>
		bool ReadTemplateClassOrStructHeader(TokenReader OriginalReader, SourceFragment Fragment)
		{
			TokenReader Reader = new TokenReader(OriginalReader);

			// Check for the template keyword
			if(Reader.Current.Text != "template")
			{
				return false;
			}

			// Create a buffer to store the template prefix
			List<Token> Tokens = new List<Token>();
			Tokens.Add(Reader.Current);

			// Check for the opening argument list
			if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines) || Reader.Current.Text != "<")
			{
				return false;
			}

			// Read the argument list, keeping track of any symbols referenced along the way
			while(Tokens[Tokens.Count - 1].Text != ">")
			{
				Tokens.Add(Reader.Current);
				if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
				{
					return false;
				}
			}

			// Get the symbol type
			SymbolType Type;
			if(Reader.Current.Text == "class")
			{
				Type = SymbolType.TemplateClass;
			}
			else if(Reader.Current.Text == "struct")
			{
				Type = SymbolType.TemplateStruct;
			}
			else
			{
				return false;
			}

			// Add the class or struct keyword
			Tokens.Add(Reader.Current);

			// Move to the name
			if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}

			// Skip over an optional DLL export declaration
			if(Reader.Current.Type == TokenType.Identifier && Reader.Current.Text.EndsWith("_API") && !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}

			// Read the class name and check it's followed by a class body or inheritance list
			string Name = Reader.Current.Text;
			if(Reader.Current.Type != TokenType.Identifier || !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}
			if(Reader.Current.Text != ":" && Reader.Current.Text != "{")
			{
				return false;
			}

			// Create the symbol.
            if(Rules.AllowSymbol(Name))
            {
				// Only allow forward declarations of templates with class and typename arguments and no defaults (ie. no enums or class names which may have to be forward-declared separately).
				string ForwardDeclaration = null;
				if(!Tokens.Any(x => x.Text == "="))
				{
					ForwardDeclaration = String.Format("{0} {1};", Token.Format(Tokens), Name);
					for(int Idx = 2; Idx < Tokens.Count - 2; Idx += 3)
					{
						if(Tokens[Idx].Text != "class" && Tokens[Idx].Text != "typename")
						{
							ForwardDeclaration = null;
							break;
						}
					}
				}
			    AddSymbol(Name, Type, ForwardDeclaration, Fragment, OriginalReader.TokenLocation);
            }

			// Move the original reader past the declaration
			OriginalReader.Set(Reader);
			return true;
		}

		/// <summary>
		/// Try to skip over a template header
		/// </summary>
		/// <param name="OriginalReader">Tokens to parse</param>
		/// <param name="Fragment">The fragment containing this symbol</param>
		bool SkipTemplateHeader(TokenReader OriginalReader)
		{
			TokenReader Reader = new TokenReader(OriginalReader);

			if(Reader.Current.Text != "template" || !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}

			while(Reader.Current.Text != ";" && Reader.Current.Text != "{")
			{
				if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
				{
					break;
				}
			}

			// Move the original reader past the declaration
			OriginalReader.Set(Reader);
			return true;
		}

		/// <summary>
		/// Adds a symbol to the lookup. If the symbol already exists and is marked as a different type, mark it as ambiguous.
		/// </summary>
		/// <param name="Name">The symbol name</param>
		/// <param name="Type">Type of the symbol</param>
		/// <param name="ForwardDeclaration">Code used to declare a forward declaration</param>
		/// <param name="Fragment">The current fragment</param>
		/// <param name="Location">Location within the file that this symbol is declared</param>
		Symbol AddSymbol(string Name, SymbolType Type, string ForwardDeclaration, SourceFragment Fragment, TextLocation Location)
		{
			// Try to find an existing symbol which matches
			foreach(Symbol ExistingSymbol in Lookup.WithKey(Name))
			{
				if(ExistingSymbol.Type == Type && ExistingSymbol.Fragment == Fragment)
				{
					return ExistingSymbol;
				}
			}

			// Otherwise create a new one
			Symbol NewSymbol = new Symbol(Name, Type, ForwardDeclaration, Fragment, Location);
			Lookup.Add(Name, NewSymbol);
			return NewSymbol;
		}

		/// <summary>
		/// Find the fragment corresponding to a given markup index
		/// </summary>
		/// <param name="File">The file being parsed</param>
		/// <param name="MarkupIdx">The preprocessor markup index within the file</param>
		/// <returns>The corresponding fragment</returns>
		SourceFragment FindFragment(SourceFile File, int MarkupIdx)
		{
			for(int Idx = 0;;Idx++)
			{
				if(MarkupIdx <= File.Fragments[Idx].MarkupMax)
				{
					return File.Fragments[Idx];
				}
			}
		}

		/// <summary>
		/// Read all the forward declarations from a file
		/// </summary>
		/// <param name="HeaderFile">The file to read from</param>
		/// <param name="SymbolToHeader">Map of symbol to the file containing it</param>
		/// <param name="Log">Writer for warnings and errors</param>
		public bool ReadForwardDeclarations(SourceFile HeaderFile, Dictionary<Symbol, SourceFile> SymbolToHeader, LineBasedTextWriter Log)
		{
			foreach (PreprocessorMarkup Markup in HeaderFile.Markup.Where(x => x.Type == PreprocessorMarkupType.Text))
			{
				TokenReader Reader = new TokenReader(HeaderFile.Location.FullName, HeaderFile.Text, Markup.Location, Markup.EndLocation);
				while(Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
				{
					if(!ReadClassOrStructForwardDeclaration(Reader, HeaderFile, SymbolToHeader, Log) 
						&& !ReadTemplateClassOrStructForwardDeclaration(Reader, HeaderFile, SymbolToHeader, Log)
						&& !ReadEnumClassForwardDeclaration(Reader, HeaderFile, SymbolToHeader, Log)
						&& !SkipAliasDeclaration(Reader))
					{
						Log.WriteLine("{0}({1}): error: invalid forward declaration - '{2}'", HeaderFile.Location, Reader.CurrentLine + 1, HeaderFile.Text[Reader.CurrentLine]);
						return false;
					}
				}
			}
			return true;
		}

		/// <summary>
		/// Parse the forward declaration of a class or struct
		/// </summary>
		/// <param name="OriginalReader">The current token reader</param>
		/// <param name="HeaderFile">The file to read from</param>
		/// <param name="SymbolToHeader">Map of symbol to the file containing it</param>
		/// <param name="Log">Writer for warnings and errors</param>
		/// <returns>True if a forward declaration was read, false otherwise</returns>
		bool ReadClassOrStructForwardDeclaration(TokenReader OriginalReader, SourceFile HeaderFile, Dictionary<Symbol, SourceFile> SymbolToHeader, LineBasedTextWriter Log)
		{
			SymbolType Type;
			if(OriginalReader.Current.Text == "class")
			{
				Type = SymbolType.Class;
			}
			else if(OriginalReader.Current.Text == "struct")
			{
				Type = SymbolType.Struct;
			}
			else
			{
				return false;
			}

			TokenReader Reader = new TokenReader(OriginalReader);
			if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines) || Reader.Current.Type != TokenType.Identifier)
			{
				return false;
			}

			Token Identifier = Reader.Current;
			if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines) || Reader.Current.Text != ";")
			{
				return false;
			}

			AddForwardDeclaration(HeaderFile, Identifier.Text, Type, SymbolToHeader, Log);
			OriginalReader.Set(Reader);
			return true;
		}

		/// <summary>
		/// Parse the forward declaration of an enum class
		/// </summary>
		/// <param name="OriginalReader">The current token reader</param>
		/// <param name="HeaderFile">The file to read from</param>
		/// <param name="SymbolToHeader">Map of symbol to the file containing it</param>
		/// <param name="Log">Writer for warnings and errors</param>
		/// <returns>True if a forward declaration was read, false otherwise</returns>
		bool ReadEnumClassForwardDeclaration(TokenReader OriginalReader, SourceFile HeaderFile, Dictionary<Symbol, SourceFile> SymbolToHeader, LineBasedTextWriter Log)
		{
			if(OriginalReader.Current.Text != "enum")
			{
				return false;
			}

			TokenReader Reader = new TokenReader(OriginalReader);
			if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}
			if(Reader.Current.Text == "class" && !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}
			if(Reader.Current.Type != TokenType.Identifier)
			{
				return false;
			}

			Token Identifier = Reader.Current;
			while(Reader.Current.Text != ";")
			{
				if(Reader.Current.Text == "{" || !Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
				{
					return false;
				}
			}

			AddForwardDeclaration(HeaderFile, Identifier.Text, SymbolType.Enumeration, SymbolToHeader, Log);
			OriginalReader.Set(Reader);
			return true;
		}

		/// <summary>
		/// Parse the forward declaration of a template class or struct
		/// </summary>
		/// <param name="OriginalReader">The current token reader</param>
		/// <param name="HeaderFile">The file to read from</param>
		/// <param name="SymbolToHeader">Map of symbol to the file containing it</param>
		/// <param name="Log">Writer for warnings and errors</param>
		/// <returns>True if a forward declaration was read, false otherwise</returns>
		bool ReadTemplateClassOrStructForwardDeclaration(TokenReader OriginalReader, SourceFile HeaderFile, Dictionary<Symbol, SourceFile> SymbolToHeader, LineBasedTextWriter Log)
		{
			if(OriginalReader.Current.Text != "template")
			{
				return false;
			}

			TokenReader Reader = new TokenReader(OriginalReader);
			if (!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				return false;
			}

			bool bMoveNext = true;
			if(!SkipTemplateArguments(Reader, ref bMoveNext) || !bMoveNext)
			{
				return false;
			}

			SymbolType Type;
			if (Reader.Current.Text == "class")
			{
				Type = SymbolType.TemplateClass;
			}
			else if(Reader.Current.Text == "struct")
			{
				Type = SymbolType.TemplateStruct;
			}
			else
			{
				return false;
			}

			if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines) || Reader.Current.Type != TokenType.Identifier)
			{
				return false;
			}

			Token Identifier = Reader.Current;
			if(!Reader.MoveNext(TokenReaderContext.IgnoreNewlines) || Reader.Current.Text != ";")
			{
				return false;
			}

			OriginalReader.Set(Reader);
			AddForwardDeclaration(HeaderFile, Identifier.Text, Type, SymbolToHeader, Log);
			return true;
		}

		/// <summary>
		/// Skip past a template argument list
		/// </summary>
		/// <param name="Reader"></param>
		/// <param name="bMoveNext"></param>
		/// <returns></returns>
		static bool SkipTemplateArguments(TokenReader Reader, ref bool bMoveNext)
		{
			if(Reader.Current.Text != "<")
			{
				return false;
			}

			int Depth = 1;
			for(;;)
			{
				bMoveNext = Reader.MoveNext(TokenReaderContext.IgnoreNewlines);
				if(!bMoveNext)
				{
					return false;
				}
				if(Reader.Current.Text == "<")
				{
					Depth++;
				}
				if(Reader.Current.Text == ">")
				{
					if(--Depth == 0) break;
				}
			}

			bMoveNext = Reader.MoveNext(TokenReaderContext.IgnoreNewlines);
			return bMoveNext;
		}

		/// <summary>
		/// Skip over the declaration of an alias
		/// </summary>
		/// <param name="OriginalReader">The current token reader</param>
		/// <returns>True if a forward declaration was read, false otherwise</returns>
		bool SkipAliasDeclaration(TokenReader OriginalReader)
		{
			TokenReader Reader = new TokenReader(OriginalReader);

			if(Reader.Current.Text == "template")
			{
				if (!Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
				{
					return false;
				}

				bool bMoveNext = true;
				if (!SkipTemplateArguments(Reader, ref bMoveNext) || !bMoveNext)
				{
					return false;
				}
			}

			if(Reader.Current.Text != "using")
			{
				return false;
			}

			while(Reader.MoveNext(TokenReaderContext.IgnoreNewlines))
			{
				if(Reader.Current.Text == ";")
				{
					break;
				}
			}

			OriginalReader.Set(Reader);
			return true;
		}

		/// <summary>
		/// Adds a forward declaration to a map
		/// </summary>
		/// <param name="HeaderFile"></param>
		/// <param name="SymbolName"></param>
		/// <param name="Type"></param>
		/// <param name="SymbolToHeader"></param>
		/// <param name="Log"></param>
		void AddForwardDeclaration(SourceFile HeaderFile, string SymbolName, SymbolType Type, Dictionary<Symbol, SourceFile> SymbolToHeader, LineBasedTextWriter Log)
		{
			foreach (Symbol Symbol in Lookup.WithKey(SymbolName))
			{
				if (Symbol.Type == Type)
				{
					SourceFile ExistingHeaderFile;
					if (SymbolToHeader.TryGetValue(Symbol, out ExistingHeaderFile) && ExistingHeaderFile != HeaderFile)
					{
						Log.WriteLine("warning: Symbol '{0}' was forward declared in '{1}' and '{2}'", Symbol.Name, HeaderFile.Location.GetFileName(), ExistingHeaderFile.Location.GetFileName());
					}
					else
					{
						SymbolToHeader[Symbol] = HeaderFile;
					}
				}
			}
		}

		/// <summary>
		/// Finds all the imported symbols from the given fragment
		/// </summary>
		/// <param name="Fragment">Fragment to find symbols for</param>
		/// <returns>Array of unique symbols for this fragment, which are not already forward-declared</returns>
		public Dictionary<Symbol, SymbolReferenceType> FindReferences(SourceFragment Fragment)
		{
			Dictionary<Symbol, SymbolReferenceType> References = new Dictionary<Symbol, SymbolReferenceType>();

			TextBuffer Text = Fragment.File.Text;
			if (Text != null && Fragment.MarkupMax > Fragment.MarkupMin)
			{
				for (int MarkupIdx = Fragment.MarkupMin; MarkupIdx < Fragment.MarkupMax; MarkupIdx++)
				{
					PreprocessorMarkup Markup = Fragment.File.Markup[MarkupIdx];
					if (Markup.Type == PreprocessorMarkupType.Text)
					{
						TokenReader Reader = new TokenReader(Fragment.File.Location.FullName, Fragment.File.Text, Markup.Location, Markup.EndLocation);
						FindReferences(Reader, Fragment, References);
					}
				}
			}

			return References;
		}

		public void FindReferences(TokenReader Reader, SourceFragment Fragment, Dictionary<Symbol, SymbolReferenceType> References)
			{
				for(bool bMoveNext = Reader.MoveNext(TokenReaderContext.IgnoreNewlines); bMoveNext; )
				{
					// Read the current token, and immediately move to the next so that we can lookahead if we need to
					Token Token = Reader.Current;
					bMoveNext = Reader.MoveNext(TokenReaderContext.IgnoreNewlines);

					// Check if the previous token is a symbol
					if(Token.Type == TokenType.Identifier)
					{
						if(Token.Text == "class" || Token.Text == "struct")
						{
							// Skip over "class", "struct" and "enum class" declarations and forward declarations. They're not actually referencing another symbol.
							if(bMoveNext && Reader.Current.Type == TokenType.Identifier && Reader.Current.Text.EndsWith("_API"))
							{
								bMoveNext = Reader.MoveNext(TokenReaderContext.IgnoreNewlines);
							}
							if(bMoveNext && Reader.Current.Type == TokenType.Identifier)
							{
								bMoveNext = Reader.MoveNext(TokenReaderContext.IgnoreNewlines);
							}
						}
						else if(IsSmartPointerClass(Token.Text) && bMoveNext && Reader.Current.Text == "<")
						{
							// Smart pointer types which do not require complete declarations
							bMoveNext = Reader.MoveNext(TokenReaderContext.IgnoreNewlines);
							if(bMoveNext && Reader.Current.Type == TokenType.Identifier)
							{
								foreach(Symbol MatchingSymbol in Lookup.WithKey(Reader.Current.Text))
								{
									if((MatchingSymbol.Type == SymbolType.Class || MatchingSymbol.Type == SymbolType.Struct) && !References.ContainsKey(MatchingSymbol) && !IgnoreSymbol(MatchingSymbol, Fragment, Reader.TokenLocation))
									{
										References[MatchingSymbol] = SymbolReferenceType.Opaque;
									}
								}
								bMoveNext = Reader.MoveNext(TokenReaderContext.IgnoreNewlines);
							}
						}
						else
						{
							// Otherwise check what type of symbol it is (if any), and whether it's referenced in an opaque way (though a pointer/reference or so on).
							foreach(Symbol MatchingSymbol in Lookup.WithKey(Token.Text))
							{
								if(IgnoreSymbol(MatchingSymbol, Fragment, Reader.TokenLocation))
								{
									continue;
								}
								else if(MatchingSymbol.Type == SymbolType.Macro)
								{
									// Add the symbol if it doesn't already exist
									if(!References.ContainsKey(MatchingSymbol))
									{
										References.Add(MatchingSymbol, SymbolReferenceType.RequiresDefinition);
									}
								}
								else if(MatchingSymbol.Type == SymbolType.Enumeration)
								{
									// Check whether we're using the type in an opaque way, or we actually require the definition
									if(!bMoveNext || Reader.Current.Text == "::")
									{
										References[MatchingSymbol] = SymbolReferenceType.RequiresDefinition;
									}
									else if(!References.ContainsKey(MatchingSymbol))
									{
										References[MatchingSymbol] = SymbolReferenceType.Opaque;
									}
								}
								else if(MatchingSymbol.Type == SymbolType.Class || MatchingSymbol.Type == SymbolType.Struct)
								{
									// Check whether we're declaring a pointer to this type, and just need a forward declaration
									if(!bMoveNext || (Reader.Current.Text != "*" && Reader.Current.Text != "&"))
									{
										References[MatchingSymbol] = SymbolReferenceType.RequiresDefinition;
									}
									else if(!References.ContainsKey(MatchingSymbol))
									{
										References[MatchingSymbol] = SymbolReferenceType.Opaque;
									}
								}
								else if(MatchingSymbol.Type == SymbolType.TemplateClass || MatchingSymbol.Type == SymbolType.TemplateStruct)
								{
									// Check whether we're declaring a pointer to this type, and just need a forward declaration
									TokenReader Lookahead = new TokenReader(Reader);
									bool bLookaheadMoveNext = bMoveNext;
									if(!bMoveNext || !SkipTemplateArguments(Lookahead, ref bLookaheadMoveNext) || (Lookahead.Current.Text != "*" && Lookahead.Current.Text != "&"))
									{
										References[MatchingSymbol] = SymbolReferenceType.RequiresDefinition;
									}
									else if(!References.ContainsKey(MatchingSymbol))
									{
										References[MatchingSymbol] = SymbolReferenceType.Opaque;
									}
								}
							}
						}
					}
				}
			}

		/// <summary>
		/// Determines whether a symbol reference should be ignored, because it's to an item defined above in the same file
		/// </summary>
		/// <param name="Symbol">The referenced symbol</param>
		/// <param name="Fragment">The current fragment being parsed</param>
		/// <param name="Location">Position within the current fragment</param>
		/// <returns>True if the symbol should be ignored</returns>
		static bool IgnoreSymbol(Symbol Symbol, SourceFragment Fragment, TextLocation Location)
		{
			if(Symbol.Fragment.File == Fragment.File && Location > Symbol.Location)
			{
				return true;
			}
			else
			{
				return false;
			}
		}

		/// <summary>
		/// Heuristic to determine if the given token looks like the name of a smart pointer class
		/// </summary>
		/// <param name="Token">The token to check</param>
		/// <returns>True if the given token appears to be the name of a smart pointer class</returns>
		static bool IsSmartPointerClass(string Token)
		{
			return Token.Length >= 5 && Token.StartsWith("T") && Char.IsUpper(Token[1]) && Token.EndsWith("Ptr");
		}
	}
}
