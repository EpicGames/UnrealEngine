// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Flags for a single token
	/// </summary>
	[Flags]
	enum TokenFlags
	{
		/// <summary>
		/// No flags
		/// </summary>
		None = 0x00,

		/// <summary>
		/// The token has space before it
		/// </summary>
		HasLeadingSpace = 0x01,

		/// <summary>
		/// Do not replace any instances of this token with the corresponding macro.
		/// </summary>
		DisableExpansion = 0x02,
	}

	/// <summary>
	/// Enumeration for token types
	/// </summary>
	enum TokenType
	{
		LeftParen,
		RightParen,
		Comma,
		Identifier,
		NumericLiteral,
		StringLiteral,
		CharacterLiteral,
		Symbol,
		StringOfTokens,
		SystemInclude,
		Placemarker,
		Unknown
	}

	/// <summary>
	/// Single lexical token
	/// </summary>
	[DebuggerDisplay("{Text}")]
	struct Token
	{
		/// <summary>
		/// Contents of this token
		/// </summary>
		public readonly string Text;

		/// <summary>
		/// The type of this token
		/// </summary>
		public readonly TokenType Type;

		/// <summary>
		/// Properties of this token
		/// </summary>
		public readonly TokenFlags Flags;

		/// <summary>
		/// Constructor
		/// </summary>
		public Token(string InText, TokenType InType, TokenFlags InFlags)
		{
			Text = InText;
			Type = InType;
			Flags = InFlags;
		}

		/// <summary>
		/// Accessor for whether this token has leading whitespace
		/// </summary>
		public bool HasLeadingSpace
		{
			get { return (Flags & TokenFlags.HasLeadingSpace) != 0; }
		}

		/// <summary>
		/// Tests whether this token matches the given string
		/// </summary>
		/// <param name="Token">Token to compare against</param>
		/// <returns>True if the token matches, false otherwise</returns>
		public bool Is(string Token)
		{
			return Text == Token;
		}

		/// <summary>
		/// Tests whether this token is a string
		/// </summary>
		/// <returns>True if this token is a string</returns>
		public bool IsString()
		{
			return Text.Length >= 2 && Text.StartsWith("\"") && Text.EndsWith("\"");
		}

		/// <summary>
		/// Tests whether this token represents a system includes, in angle brackets
		/// </summary>
		/// <returns>True if this token is a system include</returns>
		public bool IsSystemInclude()
		{
			return Text.StartsWith("<") && Text.EndsWith(">");
		}

		/// <summary>
		/// Concatenate a sequence of tokens into a string
		/// </summary>
		/// <param name="Tokens">The sequence of tokens to concatenate</param>
		/// <returns>String containing the concatenated tokens</returns>
		public static string Format(IEnumerable<Token> Tokens)
		{
			StringBuilder Result = new StringBuilder();

			IEnumerator<Token> Enumerator = Tokens.GetEnumerator();
			if(Enumerator.MoveNext())
			{
				Result.Append(Enumerator.Current.Text);

				Token LastToken = Enumerator.Current;
				while(Enumerator.MoveNext())
				{
					Token Token = Enumerator.Current;
					if(Token.HasLeadingSpace && (Token.Type != TokenType.LeftParen || LastToken.Type != TokenType.Identifier || LastToken.Text != "__pragma"))
					{
						Result.Append(" ");
					}
					Result.Append(Token.Text);
					LastToken = Token;
				}
			}

			return Result.ToString();
		}

		/// <summary>
		/// Concatenate two tokens together
		/// </summary>
		/// <param name="FirstToken">The first token</param>
		/// <param name="SecondToken">The second token</param>
		/// <returns>The combined token</returns>
		public static Token Concatenate(Token FirstToken, Token SecondToken)
		{
			string Text = FirstToken.Text + SecondToken.Text;

			TokenReader Reader = new TokenReader("concatenated", Text);
			if(Reader.MoveNext())
			{
				TokenType Type = Reader.Current.Type;
				return new Token(Text, Type, FirstToken.Flags & TokenFlags.HasLeadingSpace);
			}

			return new Token(Text, TokenType.Unknown, FirstToken.Flags & TokenFlags.HasLeadingSpace);
		}
	}
}
