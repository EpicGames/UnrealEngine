// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using System;
using System.Collections;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Flags to control the type of valid tokens read
	/// </summary>
	[Flags]
	enum TokenReaderContext
	{
		/// <summary>
		/// Allow any normal tokens
		/// </summary>
		Default = 0x0,

		/// <summary>
		/// Allow system include tokens in this context
		/// </summary>
		IncludeDirective = 0x01,

		/// <summary>
		/// An string of tokens suitable for the #error directive
		/// </summary>
		TokenString = 0x02,

		/// <summary>
		/// Don't return newline tokens
		/// </summary>
		IgnoreNewlines = 0x04,
	}

	/// <summary>
	/// Exception thrown when an invalid token is encountered
	/// </summary>
	class TokenException : Exception
	{
		public string FileName { get; }
		public TextLocation Location { get; }
		public string Description { get; }

		public TokenException(string FileName, TextLocation Location, string Description)
			: base($"{FileName}({Location.LineIdx + 1}): error: {Description}")
		{
			this.Description = Description;
		}
	}

	/// <summary>
	/// Tokenizer for C++ source files. Provides functions for navigating a source file skipping whitespace, comments, and so on when required.
	/// </summary>
	class TokenReader : IEnumerator<Token>
	{
		/// <summary>
		/// The current file
		/// </summary>
		public string FileName { get; }

		/// <summary>
		/// The current text buffer being read from
		/// </summary>
		TextBuffer Text;

		/// <summary>
		/// The current token
		/// </summary>
		Token CurrentToken;

		/// <summary>
		/// The location at which to stop reading
		/// </summary>
		TextLocation EndLocation;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Text">The text to tokenize</param>
		public TokenReader(string FileName, string Text)
			: this(FileName, TextBuffer.FromString(Text), TextLocation.Origin)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Text">The text buffer to tokenize</param>
		/// <param name="Location">Initial location to start reading from</param>
		public TokenReader(string FileName, TextBuffer Text, TextLocation Location)
			: this(FileName, Text, Location, Text.End)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Text">The text buffer to tokenize</param>
		/// <param name="Location">Initial location to start reading from</param>
		/// <param name="EndLocation">The end location to read from</param>
		public TokenReader(string FileName, TextBuffer Text, TextLocation Location, TextLocation EndLocation)
		{
			this.FileName = FileName;
			this.Text = Text;
			this.TokenWhitespaceLocation = Location;
			this.TokenLocation = Location;
			this.TokenEndLocation = Location;
			this.EndLocation = EndLocation;
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="Other">Token reader to copy from</param>
		public TokenReader(TokenReader Other)
		{
			Set(Other);
		}

		/// <summary>
		/// Sets the state of this reader to the same as another reader
		/// </summary>
		/// <param name="Other">The TokenReader to copy</param>
		public void Set(TokenReader Other)
		{
			this.Text = Other.Text;
			this.CurrentToken = Other.CurrentToken;
			this.TokenWhitespaceLocation = Other.TokenWhitespaceLocation;
			this.TokenLocation = Other.TokenLocation;
			this.TokenEndLocation = Other.TokenEndLocation;
			this.EndLocation = Other.EndLocation;
		}

		/// <summary>
		/// Returns the current token
		/// </summary>
		public Token Current
		{
			get { return CurrentToken; }
		}

		/// <summary>
		/// Returns the current line
		/// </summary>
		public int CurrentLine
		{
			get { return TokenLocation.LineIdx; }
		}

		/// <summary>
		/// Returns the start location of the whitespace before the current token
		/// </summary>
		public TextLocation TokenWhitespaceLocation
		{
			get; private set;
		}

		/// <summary>
		/// The location of the current token
		/// </summary>
		public TextLocation TokenLocation
		{
			get; private set;
		}

		/// <summary>
		/// The position at the end of the token
		/// </summary>
		public TextLocation TokenEndLocation
		{
			get; private set;
		}

		/// <summary>
		/// Untyped implementation of Current for IEnumerator.
		/// </summary>
		object IEnumerator.Current
		{
			get { return Current; }
		}

		/// <summary>
		/// Override of IEnumerator.Dispose. Not required.
		/// </summary>
		void IDisposable.Dispose()
		{
		}

		/// <summary>
		/// Move to the next token
		/// </summary>
		/// <returns>True if the reader could move to the next token, false otherwise</returns>
		public bool MoveNext()
		{
			return MoveNext(0);
		}

		/// <summary>
		/// Move to the next token
		/// </summary>
		/// <param name="Context">Flags indicating valid tokens in the current context</param>
		/// <returns>True if the reader could move to the next token, false otherwise</returns>
		public bool MoveNext(TokenReaderContext Context)
		{
			int LineIdx = TokenEndLocation.LineIdx;
			int ColumnIdx = TokenEndLocation.ColumnIdx;

			// Skip past the leading whitespace
			TokenWhitespaceLocation = new TextLocation(LineIdx, ColumnIdx);
			bool bHasWhitespace = SkipWhitespace(Text, ref LineIdx, ref ColumnIdx, Context.HasFlag(TokenReaderContext.IgnoreNewlines));

			// Check we haven't reached the end of the buffer
			TextLocation CurrentLocation = new TextLocation(LineIdx, ColumnIdx);
			if(CurrentLocation >= EndLocation)
			{
				TokenLocation = EndLocation;
				TokenEndLocation = EndLocation;
				return false;
			}

			// Read the token
			TokenLocation = new TextLocation(LineIdx, ColumnIdx);
			bool bResult = ReadToken(FileName, Text, ref LineIdx, ref ColumnIdx, bHasWhitespace? TokenFlags.HasLeadingSpace : TokenFlags.None, Context, out CurrentToken);
			TokenEndLocation = new TextLocation(LineIdx, ColumnIdx);
			return bResult;
		}

		/// <summary>
		/// Move past the next token after a newline
		/// </summary>
		/// <returns>True if we were able to find another token</returns>
		public bool MoveToNextLine()
		{
			for (; ; )
			{
				if (Current.Text == "\n")
				{
					return MoveNext();
				}
				if (!MoveNext())
				{
					return false;
				}
			}
		}

		/// <summary>
		/// Definition of IEnumerator.Reset(). Not supported.
		/// </summary>
		void IEnumerator.Reset()
		{
			throw new NotSupportedException();
		}

		/// <summary>
		/// List of multi-character symbolic tokens that we want to parse out
		/// </summary>
		static readonly string[] SymbolicTokens =
		{
			"==",
			"!=",
			"<=",
			">=",
			"++",
			"--",
			"->",
			"::",
			"&&",
			"||",
			"##",
			"<<",
			">>",
			"...",
		};

		/// <summary>
		/// Advances the given position past any horizontal whitespace or comments
		/// </summary>
		/// <param name="LineIdx">The initial line index</param>
		/// <param name="ColumnIdx">The initial column index</param>
		/// <param name="bIncludingNewlines">Whether to include  stop if reaching a newline character</param>
		/// <returns>True if there was whitespace</returns>
		static bool SkipWhitespace(TextBuffer Text, ref int LineIdx, ref int ColumnIdx, bool bIncludingNewlines)
		{
			bool bHasWhitespace = false;
			while (LineIdx < Text.Lines.Length)
			{
				// Quickly skip over trivial whitespace
				string Line = Text.Lines[LineIdx];
				while (ColumnIdx < Line.Length && (Line[ColumnIdx] == ' ' || Line[ColumnIdx] == '\t' || Line[ColumnIdx] == '\v'))
				{
					ColumnIdx++;
					bHasWhitespace = true;
				}

				// Look at what's next
				char Character = Text[LineIdx, ColumnIdx];
				if (Character == '\\' && Text[LineIdx, ColumnIdx + 1] == '\n')
				{
					LineIdx++;
					ColumnIdx = 0;
				}
				else if (Character == '/' && Text[LineIdx, ColumnIdx + 1] == '/')
				{
					ColumnIdx = Text.Lines[LineIdx].Length;
					bHasWhitespace = true;
				}
				else if (Character == '/' && Text[LineIdx, ColumnIdx + 1] == '*')
				{
					ColumnIdx += 2;
					while (Text[LineIdx, ColumnIdx] != '*' || Text[LineIdx, ColumnIdx + 1] != '/')
					{
						Text.MoveNext(ref LineIdx, ref ColumnIdx);
					}
					ColumnIdx += 2;
					bHasWhitespace = true;
				}
				else if(Character == '\n' && bIncludingNewlines)
				{
					LineIdx++;
					ColumnIdx = 0;
					bHasWhitespace = true;
				} 
				else
				{
					break;
				}
			}
			return bHasWhitespace;
		}

		/// <summary>
		/// Reads a single token from a text buffer
		/// </summary>
		/// <param name="FileName">File being parsed</param>
		/// <param name="Text">The text buffer to read from</param>
		/// <param name="LineIdx">The current line index</param>
		/// <param name="ColumnIdx">The current column index</param>
		/// <param name="Flags">Flags for the new token</param>
		/// <returns>The next token, or null at the end of the file</returns>
		static bool ReadToken(string FileName, TextBuffer Text, ref int LineIdx, ref int ColumnIdx, TokenFlags Flags, TokenReaderContext Context, out Token Result)
		{
			int StartLineIdx = LineIdx;
			int StartColumnIdx = ColumnIdx;

			char Character = Text.ReadCharacter(ref LineIdx, ref ColumnIdx);
			if (Character == '\0')
			{
				Result = new Token("", TokenType.Placemarker, Flags);
				return false;
			}
			else if(Context.HasFlag(TokenReaderContext.TokenString))
			{
				// Raw token string until the end of the current line
				StringBuilder Builder = new StringBuilder();
				if(Character == '\n')
				{
					LineIdx = StartLineIdx;
					ColumnIdx = StartColumnIdx;
				}
				else
				{
					Builder.Append(Character);
					for(;;)
					{
						Character = Text[LineIdx, ColumnIdx];
						if(Character == '\n' && Builder[^1] != '\\')
						{
							break;
						}
						Builder.Append(Character);
						if(!Text.MoveNext(ref LineIdx, ref ColumnIdx))
						{
							break;
						}
					}
				}
				Result = new Token(Builder.ToString().TrimEnd(), TokenType.StringOfTokens, Flags);
				return true;
			}
			else if(Character == '\'')
			{
				// Character literal
				SkipTextLiteral(FileName, Text, ref LineIdx, ref ColumnIdx, '\'');
				Result = new Token(Text.ExtractString(StartLineIdx, StartColumnIdx, LineIdx, ColumnIdx), TokenType.CharacterLiteral, Flags);
				return true;
			}
			else if(Character == '\"')
			{
				// String literal
				SkipTextLiteral(FileName, Text, ref LineIdx, ref ColumnIdx, '\"');
				Result = new Token(Text.ExtractString(StartLineIdx, StartColumnIdx, LineIdx, ColumnIdx), TokenType.StringLiteral, Flags);
				return true;
			}
			else if ((Character >= 'a' && Character <= 'z') || (Character >= 'A' && Character <= 'Z') || Character == '_')
			{
				char FirstCharacter = Character;

				// Identifier (or text literal with prefix)
				int PrefixLength = 1;
				for (;; PrefixLength++)
				{
					Character = Text[LineIdx, ColumnIdx];
					if((Character < 'a' || Character > 'z') && (Character < 'A' || Character > 'Z') && (Character < '0' || Character > '9') && Character != '_' && Character != '$')
					{
						break;
					}
					Text.MoveNext(ref LineIdx, ref ColumnIdx);
				}

				// Check if it's a prefixed text literal
				if (Character == '\'')
				{
					Text.MoveNext(ref LineIdx, ref ColumnIdx);
					SkipTextLiteral(FileName, Text, ref LineIdx, ref ColumnIdx, '\'');
					Result = new Token(Text.ExtractString(StartLineIdx, StartColumnIdx, LineIdx, ColumnIdx), TokenType.CharacterLiteral, Flags);
					return true;
				}
				else if (Character == '\"' && PrefixLength == 1 && FirstCharacter == 'R')
				{
					SkipRawStringLiteral(FileName, Text, ref LineIdx, ref ColumnIdx);
					Result = new Token(Text.ExtractString(StartLineIdx, StartColumnIdx, LineIdx, ColumnIdx), TokenType.StringLiteral, Flags);
					return true;
				}
				else if (Character == '\"')
				{
					Text.MoveNext(ref LineIdx, ref ColumnIdx);
					SkipTextLiteral(FileName, Text, ref LineIdx, ref ColumnIdx, '\"');
					Result = new Token(Text.ExtractString(StartLineIdx, StartColumnIdx, LineIdx, ColumnIdx), TokenType.StringLiteral, Flags);
					return true;
				}
				else
				{
					Result = new Token(Text.ExtractString(StartLineIdx, StartColumnIdx, LineIdx, ColumnIdx), TokenType.Identifier, Flags);
					return true;
				}
			}
			else if ((Character >= '0' && Character <= '9') || (Character == '.' && (Text[LineIdx, ColumnIdx] >= '0' && Text[LineIdx, ColumnIdx] <= '9')))
			{
				// pp-number token
				char LastCharacter = Character;
				for (;;)
				{
					Character = Text[LineIdx, ColumnIdx];
					if((Character < 'a' || Character > 'z') && (Character < 'A' || Character > 'Z') && (Character < '0' || Character > '9') && Character != '_' && Character != '$' && Character != '\'')
					{
						if((Character != '+' && Character != '-') || (LastCharacter != 'e' && LastCharacter != 'E'))
						{
							break;
						}
					}
					LastCharacter = Text.ReadCharacter(ref LineIdx, ref ColumnIdx);
				}
				Result = new Token(Text.ExtractString(StartLineIdx, StartColumnIdx, LineIdx, ColumnIdx), TokenType.NumericLiteral, Flags);
				return true;
			}
			else if(Character == '<' && Context.HasFlag(TokenReaderContext.IncludeDirective))
			{
				StringBuilder Builder = new StringBuilder("<");
				while(Builder[Builder.Length - 1] != '>' && ColumnIdx < Text.Lines[LineIdx].Length)
				{
					Builder.Append(Text[LineIdx, ColumnIdx]);
					ColumnIdx++;
				}
				Result = new Token(Builder.ToString(), TokenType.SystemInclude, Flags);
				return true;
			}
			else
			{
				// Try to read a symbol
				if (ColumnIdx > 0)
				{
					for (int Idx = 0; Idx < SymbolicTokens.Length; Idx++)
					{
						string SymbolicToken = SymbolicTokens[Idx];
						for (int Length = 0; Text[LineIdx, ColumnIdx + Length - 1] == SymbolicToken[Length]; Length++)
						{
							if (Length + 1 == SymbolicToken.Length)
							{
								ColumnIdx += Length;
								Result = new Token(SymbolicToken, TokenType.Symbol, Flags);
								return true;
							}
						}
					}
				}

				// Otherwise just return a single character
				TokenType Type;
				switch(Character)
				{
					case '(':
						Type = TokenType.LeftParen;
						break;
					case ')':
						Type = TokenType.RightParen;
						break;
					case ',':
						Type = TokenType.Comma;
						break;
					default:
						Type = TokenType.Symbol;
						break;
				}
				Result = new Token(Character.ToString(), Type, Flags);
				return true;
			}
		}

		/// <summary>
		/// Skip past a text literal (a quoted character literal or string literal)
		/// </summary>
		/// <param name="FileName">The file being parsed</param>
		/// <param name="Text">The text buffer to read from</param>
		/// <param name="LineIdx">The current line index</param>
		/// <param name="ColumnIdx">The current column index</param>
		/// <param name="LastCharacter">The terminating character to look for, ignoring escape sequences</param>
		static void SkipTextLiteral(string FileName, TextBuffer Text, ref int LineIdx, ref int ColumnIdx, char LastCharacter)
		{
			for(;;)
			{
				char Character = Text.ReadCharacter(ref LineIdx, ref ColumnIdx);
				if (Character == '\0')
				{
					throw new TokenException(FileName, new TextLocation(LineIdx, ColumnIdx), "Unexpected end of file in text literal");
				}
				else if (Character == '\\')
				{
					Text.MoveNext(ref LineIdx, ref ColumnIdx);
				}
				else if (Character == LastCharacter)
				{
					break;
				}
				else if (Character == '\n')
				{
					throw new TokenException(FileName, new TextLocation(LineIdx, ColumnIdx), "Unexpected newline in text literal");
				}
			}
		}

		/// <summary>
		/// Skip past a text literal (a quoted character literal or string literal)
		/// </summary>
		/// <param name="FileName">The file being parsed</param>
		/// <param name="Text">The text buffer to read from</param>
		/// <param name="LineIdx">The current line index</param>
		/// <param name="ColumnIdx">The current column index</param>
		static void SkipRawStringLiteral(string FileName, TextBuffer Text, ref int LineIdx, ref int ColumnIdx)
		{
			int StartColumnIdx = ColumnIdx;

			ReadOnlySpan<char> Line = Text[LineIdx].AsSpan();
			for (; ; ColumnIdx++)
			{
				if (ColumnIdx == Line.Length)
				{
					throw new TokenException(FileName, new TextLocation(LineIdx, ColumnIdx), "End of line when looking for raw text delimiter");
				}
				else if (Line[ColumnIdx] == '(')
				{
					break;
				}
			}

			ReadOnlySpan<char> Delim = Line.Slice(StartColumnIdx + 1, ColumnIdx - (StartColumnIdx + 1));
			while(LineIdx < Text.Lines.Length)
			{
				Line = Text[LineIdx].AsSpan();

				for (; ColumnIdx < Line.Length; ColumnIdx++)
				{
					if (Line[ColumnIdx] == ')' && Line.Length >= ColumnIdx + 2 + Delim.Length && Line[ColumnIdx + 1 + Delim.Length] == '\"' && Line.Slice(ColumnIdx + 1).StartsWith(Delim))
					{
						ColumnIdx += Delim.Length + 2;
						return;
					}
				}

				(LineIdx, ColumnIdx) = (LineIdx + 1, 0);
			}

			throw new TokenException(FileName, new TextLocation(LineIdx, ColumnIdx), "End of file while looking for end of raw string delimiter");
		}
	}
}
