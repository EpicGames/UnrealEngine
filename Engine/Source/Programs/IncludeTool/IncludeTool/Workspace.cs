// Copyright Epic Games, Inc. All Rights Reserved.

using IncludeTool.Support;
using EpicGames.Core;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;

namespace IncludeTool
{
	/// <summary>
	/// Represents a file in the workspace
	/// </summary>
	class WorkspaceFile
	{
		/// <summary>
		/// Location of the file
		/// </summary>
		public readonly FileReference Location;

		/// <summary>
		/// The directory containing this file
		/// </summary>
		public readonly WorkspaceDirectory Directory;

		/// <summary>
		/// Lowercase path from the root of the branch
		/// </summary>
		public readonly string NormalizedPathFromBranchRoot;

		/// <summary>
		/// The cached source file corresponding to this file
		/// </summary>
		SourceFile CachedSourceFile;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Name">Name of the file</param>
		/// <param name="Directory">The directory containing the file</param>
		public WorkspaceFile(string Name, WorkspaceDirectory Directory)
		{
			this.Location = FileReference.Combine(Directory.Location, Name);
			this.Directory = Directory;

			if(Directory.NormalizedPathFromBranchRoot != null)
			{
				this.NormalizedPathFromBranchRoot = Directory.NormalizedPathFromBranchRoot + Name.ToLowerInvariant();
			}
		}

		/// <summary>
		/// The project directory containing this file
		/// </summary>
		public WorkspaceDirectory ProjectDirectory
		{
			get { return Directory.ProjectDirectory; }
		}

		/// <summary>
		/// Returns the cached source file object corresponding to this file. Call ReadSourceFile() to return this value, populating it if necessary.
		/// </summary>
		public SourceFile SourceFile
		{
			get { return CachedSourceFile; }
		}
		
		/// <summary>
		/// Returns 
		/// </summary>
		/// <returns></returns>
		public SourceFile ReadSourceFile()
		{
			if(CachedSourceFile == null)
			{
				SourceFile NewCachedSourceFile = new SourceFile(Location, TextBuffer.FromFile(Location.FullName), Rules.GetSourceFileFlags(this));
				Interlocked.CompareExchange(ref CachedSourceFile, NewCachedSourceFile, null);
			}
			return CachedSourceFile;
		}

		/// <summary>
		/// Provide a string representation of this object for debugging
		/// </summary>
		/// <returns>String representation of this object</returns>
		public override string ToString()
		{
			return Location.ToString();
		}
	}

	/// <summary>
	/// Caches the contents of a workspace directory
	/// </summary>
	class WorkspaceDirectory
	{
		/// <summary>
		/// The location of this directory
		/// </summary>
		public readonly DirectoryReference Location;

		/// <summary>
		/// The parent directory
		/// </summary>
		public readonly WorkspaceDirectory ParentDirectory;

		/// <summary>
		/// The directory info for this directory
		/// </summary>
		DirectoryInfo Info;

		/// <summary>
		/// Files within this directory
		/// </summary>
		Dictionary<string, WorkspaceFile> NameToFile;

		/// <summary>
		/// Subdirectories within this directory
		/// </summary>
		Dictionary<string, WorkspaceDirectory> NameToDirectory;

		/// <summary>
		/// Construct a directory object with the given DirectoryInfo as the root
		/// </summary>
		/// <param name="Info">Info for the directory to represent</param>
		/// <param name="bIsBranchRoot">Whether this directory is the root of the branch</param>
		public WorkspaceDirectory(DirectoryInfo Info)
		{
			if(Info.Parent != null)
			{
				throw new Exception("WorkspaceDirectory may only be constructed directly for a root directory");
			}

			this.Location = new DirectoryReference(Info.FullName);
			this.ParentDirectory = null;
			this.Info = Info;
		}

		/// <summary>
		/// Construct a directory object, as a child of the given parent directory
		/// </summary>
		/// <param name="Info">Info for the directory to represent</param>
		/// <param name="ParentDirectory">The </param>
		WorkspaceDirectory(DirectoryInfo Info, WorkspaceDirectory ParentDirectory)
		{
			this.Location = DirectoryReference.Combine(ParentDirectory.Location, Info.Name);
			this.ParentDirectory = ParentDirectory;
			this.Info = Info;

			if(ParentDirectory.NormalizedPathFromBranchRoot != null)
			{
				NormalizedPathFromBranchRoot = ParentDirectory.NormalizedPathFromBranchRoot + Info.Name.ToLowerInvariant() + "/";
			}
		}

		/// <summary>
		/// Caches the contents of this directory
		/// </summary>
		public void CacheFiles()
		{
			// Check if we need to update the file list
			if(NameToFile == null)
			{
				// Add all the files in this directory
				Dictionary<string, WorkspaceFile> NewNameToFile = new Dictionary<string, WorkspaceFile>(StringComparer.OrdinalIgnoreCase);
				foreach (FileInfo FileInfo in Info.EnumerateFiles())
				{
					WorkspaceFile File = new WorkspaceFile(FileInfo.Name, this);
					NewNameToFile.Add(FileInfo.Name, File);
				}
				Interlocked.CompareExchange(ref NameToFile, NewNameToFile, null);

				// Update the project directory if we have one. For programs, we treat the directory containing .target.cs files as the project, because they
				// do not overlap with other programs or projects.
				if(NormalizedPathFromBranchRoot != null && ProjectDirectory == null)
				{
					if(NormalizedPathFromBranchRoot.StartsWith("/engine/"))
					{
						if(NormalizedPathFromBranchRoot.StartsWith("/engine/source/programs/") && Files.Any(x => x.NormalizedPathFromBranchRoot.EndsWith(".target.cs")))
						{
							ProjectDirectory = this;
						}
					}
					else
					{
						if(Files.Any(x => x.NormalizedPathFromBranchRoot.EndsWith(".uproject")))
						{
							ProjectDirectory = this;
						}
					}
				}
			}
		}

		/// <summary>
		/// Caches the directories in this directory
		/// </summary>
		public void CacheDirectories()
		{
			if(NameToDirectory == null)
			{
				Dictionary<string, WorkspaceDirectory> NewNameToDirectory = new Dictionary<string, WorkspaceDirectory>(StringComparer.OrdinalIgnoreCase);
				foreach(DirectoryInfo DirectoryInfo in Info.EnumerateDirectories())
				{
					WorkspaceDirectory Directory = new WorkspaceDirectory(DirectoryInfo, this);
					NewNameToDirectory.Add(DirectoryInfo.Name, Directory);
				}
				Interlocked.CompareExchange(ref NameToDirectory, NewNameToDirectory, null);
			}
		}

		/// <summary>
		/// Tries to get a file of the given name from this directory
		/// </summary>
		/// <param name="Name">The name to look for</param>
		/// <param name="Result">If successful, receives the file object</param>
		/// <returns>True if the file was found</returns>
		public bool TryGetFile(string Name, out WorkspaceFile Result)
		{
			CacheFiles();
			return NameToFile.TryGetValue(Name, out Result);
		}

		/// <summary>
		/// Tries to get a directory of the given name from this directory
		/// </summary>
		/// <param name="Name">The name of the directory to look for</param>
		/// <param name="Result">If successful, receives the directory object</param>
		/// <returns>True if the directory was found</returns>
		public bool TryGetDirectory(string Name, out WorkspaceDirectory Result)
		{
			if(Name.Length == 1 && Name[0] == '.')
			{
				Result = this;
				return true;
			}
			else if(Name.Length == 2 && Name == ".." && ParentDirectory != null)
			{
				Result = ParentDirectory;
				return true;
			}
			else
			{
				CacheDirectories();
				return NameToDirectory.TryGetValue(Name, out Result);
			}
		}

		/// <summary>
		/// Marks a directory as the root of a branch. This sets the NormalizedPathFromBranchRoot property to "/", and child directories will be updated with subpaths.
		/// </summary>
		public void SetAsBranchRoot()
		{
			if(NameToDirectory != null)
			{
				throw new Exception("This directory already has cached children. SetAsBranchRoot() must be called before expanding the contents of a directory.");
			}
			NormalizedPathFromBranchRoot = "/";
		}

		/// <summary>
		/// Returns the files within this directory
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		public IEnumerable<WorkspaceFile> Files
		{
			get
			{
				CacheFiles();
				return NameToFile.Values;
			}
		}

		/// <summary>
		/// Returns the child directories within this directory
		/// </summary>
		[DebuggerBrowsable(DebuggerBrowsableState.Never)]
		public IEnumerable<WorkspaceDirectory> Directories
		{
			get
			{
				CacheDirectories();
				return NameToDirectory.Values;
			}
		}

		/// <summary>
		/// Gets the project directory corresponding to the current directory
		/// </summary>
		public WorkspaceDirectory ProjectDirectory
		{
			get;
			private set;
		}

		/// <summary>
		/// The normalized (ie. lowercase/invariant, separated by forward slashes, with a trailing slash) path of this directory from the branch root. Null if the path is outside the branch.
		/// </summary>
		public string NormalizedPathFromBranchRoot
		{
			get;
			private set;
		}

		/// <summary>
		/// Returns a string representation of this directory for debugging
		/// </summary>
		/// <returns>Path to this directory</returns>
		public override string ToString()
		{
			return Location.ToString();
		}
	}

	/// <summary>
	/// Worker class to find files in the workspace
	/// </summary>
	class WorkspaceScanner
	{
		/// <summary>
		/// The current directory being searched
		/// </summary>
		WorkspaceDirectory Directory;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="CurrentDirectory">The directory to scan</param>
		public WorkspaceScanner(WorkspaceDirectory Directory)
		{
			this.Directory = Directory;
		}

		/// <summary>
		/// Callback from the thread pool worker
		/// </summary>
		/// <param name="Queue">The queue calling this method</param>
		public void Run(ThreadPoolWorkQueue Queue)
		{
			WorkspaceDirectory CurrentDirectory = Directory;
			while(CurrentDirectory != null)
			{
				CurrentDirectory.CacheFiles();
				CurrentDirectory.CacheDirectories();

				if(CurrentDirectory.NormalizedPathFromBranchRoot != null && !Rules.SearchDirectoryForSource(Directory.NormalizedPathFromBranchRoot))
				{
					break;
				}

				WorkspaceDirectory NextDirectory = null;
				foreach(WorkspaceDirectory ChildDirectory in CurrentDirectory.Directories)
				{
					if(NextDirectory == null)
					{
						NextDirectory = ChildDirectory;
					}
					else
					{
						Queue.Enqueue(() => new WorkspaceScanner(ChildDirectory).Run(Queue));
					}
				}
				CurrentDirectory = NextDirectory;
			}
		}
	}

	/// <summary>
	/// Worker class to find files in the workspace
	/// </summary>
	class WorkspaceReferenceScanner
	{
		/// <summary>
		/// The current directory being searched
		/// </summary>
		WorkspaceDirectory ParentDirectory;

		/// <summary>
		/// The set of files that have been found
		/// </summary>
		ConcurrentBag<WorkspaceFile> FoundFiles;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="ParentDirectory">The directory to scan</param>
		/// <param name="FoundFiles">The set of files that have been found</param>
		public WorkspaceReferenceScanner(WorkspaceDirectory ParentDirectory, ConcurrentBag<WorkspaceFile> FoundFiles)
		{
			this.ParentDirectory = ParentDirectory;
			this.FoundFiles = FoundFiles;
		}

		/// <summary>
		/// Callback from the thread pool worker
		/// </summary>
		/// <param name="Queue">The queue calling this method</param>
		public void Run(ThreadPoolWorkQueue Queue)
		{
			// Add all the files in this directory
			foreach(WorkspaceFile File in ParentDirectory.Files)
			{
				if(Rules.IsSourceFile(File.NormalizedPathFromBranchRoot))
				{
					// Read the contents of the file
					if(!Rules.IsExternalHeaderPath(File.NormalizedPathFromBranchRoot))
					{
						File.ReadSourceFile();
					}

					// Add it to the output list of files
					FoundFiles.Add(File);
				}
			}

			// Add everything in subdirectories
			foreach(WorkspaceDirectory Directory in ParentDirectory.Directories)
			{
				if (Rules.SearchDirectoryForSource(Directory.NormalizedPathFromBranchRoot))
				{
					Queue.Enqueue(() => new WorkspaceReferenceScanner(Directory, FoundFiles).Run(Queue));
				}
			}
		}
	}

	/// <summary>
	/// Utility functions for finding files in the workspace
	/// </summary>
	static class Workspace
	{
		/// <summary>
		/// The branch root directory
		/// </summary>
		public static WorkspaceDirectory BranchRoot
		{
			get;
			private set;
		}

		/// <summary>
		/// List of root directories to search in
		/// </summary>
		static ConcurrentDictionary<DirectoryReference, WorkspaceDirectory> RootDirectories = new ConcurrentDictionary<DirectoryReference, WorkspaceDirectory>();

		/// <summary>
		/// Sets the location of the branch root directory
		/// </summary>
		/// <param name="Location"></param>
		/// <returns></returns>
		public static void SetBranchRoot(DirectoryReference Location)
		{
			if(RootDirectories.Count > 0)
			{
				throw new Exception("Branch root must be set before any other directory queries");
			}

			WorkspaceDirectory NewBranchRoot;
			if(!TryGetDirectoryInternal(new DirectoryInfo(Location.FullName), out NewBranchRoot))
			{
				throw new DirectoryNotFoundException(String.Format("Couldn't find directory {0}", Location.FullName));
			}

			NewBranchRoot.SetAsBranchRoot();
			BranchRoot = NewBranchRoot;
		}

		/// <summary>
		/// Caches all the files under the given directory
		/// </summary>
		/// <param name="Directory">The directory to cache</param>
		public static void CacheFiles(WorkspaceDirectory Directory)
		{
			using (ThreadPoolWorkQueue WorkQueue = new ThreadPoolWorkQueue())
			{
				WorkQueue.Enqueue(() => new WorkspaceScanner(Directory).Run(WorkQueue));
			}
		}

		/// <summary>
		/// Gets a WorkspaceFile object for the file at the given location. Throws an exception if the file does not exist.
		/// </summary>
		/// <param name="Location">Location of the file</param>
		/// <returns>The unique WorkspaceFile object for the given file</returns>
		public static WorkspaceFile GetFile(FileReference Location)
		{
			// Find the parent directory
			WorkspaceDirectory Directory = GetDirectory(Location.Directory);

			// Find the file within it
			WorkspaceFile Result;
			if(!Directory.TryGetFile(Location.GetFileName(), out Result))
			{
				throw new DirectoryNotFoundException(String.Format("Couldn't find file {0}", Location.FullName));
			}
			return Result;
		}

		/// <summary>
		/// Gets a WorkspaceDirectory object representing the given location
		/// </summary>
		/// <param name="Location"></param>
		/// <returns></returns>
		public static WorkspaceDirectory GetDirectory(DirectoryReference Location)
		{
			WorkspaceDirectory Result;
			if(!TryGetDirectoryInternal(new DirectoryInfo(Location.FullName), out Result))
			{
				throw new DirectoryNotFoundException(String.Format("Couldn't find directory {0}", Location.FullName));
			}
			return Result;
		}

		/// <summary>
		/// Tries to get the path to a directory, recursively adding all the directories above it if necessary
		/// </summary>
		/// <param name="Info">Info for the directory to find</param>
		/// <param name="Result">On success, receives the directory</param>
		/// <returns>True if the directory exists</returns>
		static bool TryGetDirectoryInternal(DirectoryInfo Info, out WorkspaceDirectory Result)
		{
			DirectoryInfo ParentInfo = Info.Parent;
			if(ParentInfo == null)
			{
				// Try to find an existing root directory that matches this directory, or add a new one
				DirectoryReference Location = new DirectoryReference(Info);
				if(!RootDirectories.TryGetValue(Location, out Result))
				{
					Result = RootDirectories.GetOrAdd(Location, new WorkspaceDirectory(Info));
				}
				return true;
			}
			else
			{
				// Get the parent directory
				WorkspaceDirectory ParentDirectory;
				if(!TryGetDirectoryInternal(ParentInfo, out ParentDirectory))
				{
					Result = null;
					return false;
				}

				// Look for a directory within this directory
				return ParentDirectory.TryGetDirectory(Info.Name, out Result);
			}
		}
	}
}
